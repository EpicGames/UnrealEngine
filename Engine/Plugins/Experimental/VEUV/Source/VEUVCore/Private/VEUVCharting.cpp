// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVCharting.h"
#include "VEUVGeometry.h"
#include "Async/ParallelFor.h"

/**
 * TODO: The actual charting (not rasterization) is mostly a placeholder.
*/

struct FRasterizedUV
{
	static constexpr int32 BinPixelSize = 8;
	
	int32 Width = 0;
	int32 Height = 0;

	struct FRasterizedTriangle
	{
		int32 FaceIndex = 0;
		Eigen::Vector2f UV0, UV1, UV2;
	};

	struct FBin
	{
		TArray<FRasterizedTriangle> Triangles;
	};

	TArray<FBin> Bins;
	TArray<uint64> BinOccupancy;
};

struct FRasterizedChartTraversalContext : VEUV::FChartTraversalContext
{
	Eigen::Vector2f RUVMin;
	Eigen::Vector2f RUVMax;
	TArray<Eigen::Vector2f> CachedVertexUVs;
	TArray<FRasterizedUV> VoxelRUVs;
};

struct FUVRegion
{
	Eigen::Vector2f UVMin = Eigen::Vector2f::Constant(FLT_MAX);
	Eigen::Vector2f UVMax = Eigen::Vector2f::Constant(-FLT_MAX);
};

static const Eigen::Vector3i GChartDirections[] = {
	Eigen::Vector3i(-1,  0,  0), Eigen::Vector3i( 1,  0,  0),
	Eigen::Vector3i( 0, -1,  0), Eigen::Vector3i( 0,  1,  0),
	Eigen::Vector3i( 0,  0, -1), Eigen::Vector3i( 0,  0,  1),
};

static bool RegionIntersection(const FUVRegion& A, const FUVRegion& B, float Eps)
{
	if (A.UVMin.x() + Eps > B.UVMax.x() || A.UVMax.x() < B.UVMin.x() + Eps ||
		A.UVMin.y() + Eps > B.UVMax.y() || A.UVMax.y() < B.UVMin.y() + Eps)
	{
		return false;
	}
	
	return true;
}

static FUVRegion GetRegion(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Solve,
	const VEUV::FVoxelData& Voxel,
	const VEUV::FUVNormalization& Norm)
{
	FUVRegion Region;
	
	// TODO: Refactor
	for (const VEUV::FSampleData& Sample : Voxel.Samples)
	{
		Eigen::Vector2f UV = Norm.Transform(Grid.SampleUV(Solve.SolvedUV, &Voxel - Grid.Voxels.GetData(), Sample.Position));
		Region.UVMin = Region.UVMin.cwiseMin(UV);
		Region.UVMax = Region.UVMax.cwiseMax(UV);
	}
	
	return Region;
}

static void RasterizeSamples(
	const FRasterizedChartTraversalContext& Context,
	const VEUV::FVoxelData& Voxel,
	const TArray<Eigen::Vector3i>& Faces,
	FRasterizedUV& RUV)
{
	if (Voxel.Samples.IsEmpty())
	{
		return;
	}

	// Rasterized bins
	int32 BinsX = RUV.Width / FRasterizedUV::BinPixelSize;
	int32 BinsY = RUV.Height / FRasterizedUV::BinPixelSize;

	// Normalize samples within the region
	Eigen::Vector2f RUVExtent = Context.RUVMax - Context.RUVMin;

	for (int32 FaceIdx : Voxel.Faces)
	{
		const Eigen::Vector3i& Face = Faces[FaceIdx];

		// Face UVs
		Eigen::Vector2f UV0 = Context.CachedVertexUVs[Face.x()];
		Eigen::Vector2f UV1 = Context.CachedVertexUVs[Face.y()];
		Eigen::Vector2f UV2 = Context.CachedVertexUVs[Face.z()];

		// Normalize them
		UV0 = (UV0 - Context.RUVMin).cwiseQuotient(RUVExtent);
		UV1 = (UV1 - Context.RUVMin).cwiseQuotient(RUVExtent);
		UV2 = (UV2 - Context.RUVMin).cwiseQuotient(RUVExtent);

		// Rasterized bounds
		Eigen::Vector2f MinTri = UV0.cwiseMin(UV1).cwiseMin(UV2);
		Eigen::Vector2f MaxTri = UV0.cwiseMax(UV1).cwiseMax(UV2);

		// To bins
		Eigen::Vector2f MinTriBin = MinTri.cwiseProduct(Eigen::Vector2f(BinsX, BinsY));
		Eigen::Vector2f MaxTriBin = MaxTri.cwiseProduct(Eigen::Vector2f(BinsX, BinsY));

		// First, add the triangle to its bins
		for (int32 BinY = FMath::Max(0, (int32)std::floor(MinTriBin.y())); BinY <= FMath::Min((int32)std::ceil(MaxTriBin.y()), BinsY - 1); BinY++)
		{
			for (int32 BinX = FMath::Max(0, (int32)std::floor(MinTriBin.x())); BinX <= FMath::Min((int32)std::ceil(MaxTriBin.x()), BinsX - 1); BinX++)
			{
				FRasterizedUV::FRasterizedTriangle Tri;
				Tri.FaceIndex = FaceIdx;
				Tri.UV0 = UV0; Tri.UV1 = UV1; Tri.UV2 = UV2;
				RUV.Bins[BinY * BinsX + BinX].Triangles.Add(Tri);
			}
		}

		// To texels
		Eigen::Vector2f MinTriPx = MinTri.cwiseProduct(Eigen::Vector2f(RUV.Width, RUV.Height));
		Eigen::Vector2f MaxTriPx = MaxTri.cwiseProduct(Eigen::Vector2f(RUV.Width, RUV.Height));

		// Hoisted denom
		Eigen::Vector2f EAB = UV1 - UV0;
		Eigen::Vector2f EAC = UV2 - UV0;
		float D = EAC.squaredNorm() * EAB.squaredNorm() - EAB.dot(EAC) * EAB.dot(EAC);

		for (int32 Y = (int32)std::floor(MinTriPx.y()); Y < (int32)std::ceil(MaxTriPx.y()); Y++)
		{
			for (int32 X = (int32)std::floor(MinTriPx.x()); X < (int32)std::ceil(MaxTriPx.x()); X++)
			{
				if (X < 0 || X >= RUV.Width || Y < 0 || Y >= RUV.Height)
				{
					continue;
				}

				// Local texel position
				Eigen::Vector2f Pos((X + 0.5f) / RUV.Width, (Y + 0.5f) / RUV.Height);
				Eigen::Vector2f P = Pos - UV0;
				
				// Barycentrics
				float U = (EAB.squaredNorm() * P.dot(EAC) - EAB.dot(EAC) * P.dot(EAB)) / D;
				float V = (EAC.squaredNorm() * P.dot(EAB) - EAB.dot(EAC) * P.dot(EAC)) / D;

				// Mark respective bin occupancy if overlapping
				if (U >= 0 && V >= 0 && U + V <= 1)
				{
					int32 BinIdx = (Y / FRasterizedUV::BinPixelSize) * BinsX + (X / FRasterizedUV::BinPixelSize);
					int32 LocalX = X % FRasterizedUV::BinPixelSize;
					int32 LocalY = Y % FRasterizedUV::BinPixelSize;
					RUV.BinOccupancy[BinIdx] |= (1ULL << (LocalY * FRasterizedUV::BinPixelSize + LocalX));
				}
			}
		}
	}
}

static bool RasterizationCheckWithBinTest(const FRasterizedUV& A, const FRasterizedUV& B)
{
	int32 BinsX = A.Width  / FRasterizedUV::BinPixelSize;
	int32 BinsY = A.Height / FRasterizedUV::BinPixelSize;

	// Traverse by bins
	for (int32 BinY = 0; BinY < BinsY; BinY++)
	{
		for (int32 BinX = 0; BinX < BinsX; BinX++)
		{
			int32 BinIdx = BinY * BinsX + BinX;

			// Check the rasterized bins, 8x8
			if (!(A.BinOccupancy[BinIdx] & B.BinOccupancy[BinIdx]))
			{
				continue;
			}

			const FRasterizedUV::FBin& BinA = A.Bins[BinIdx];
			const FRasterizedUV::FBin& BinB = B.Bins[BinIdx];

			// No triangles?
			if (BinA.Triangles.IsEmpty() || BinB.Triangles.IsEmpty())
			{
				continue;
			}

			// Slow tri-by-tri comparison
			for (const auto& TriA : BinA.Triangles)
			{
				for (const auto& TriB : BinB.Triangles)
				{
					if (TriA.FaceIndex == TriB.FaceIndex)
					{
						continue;
					}

					if (VEUV::Geometry::TriTriTest(TriA.UV0, TriA.UV1, TriA.UV2, TriB.UV0, TriB.UV1, TriB.UV2, 0))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

static void CombineRUV(FRasterizedUV& A, const FRasterizedUV& B)
{
	// Combine bin masks
	for (int32 i = 0; i < A.BinOccupancy.Num(); i++)
	{
		A.BinOccupancy[i] |= B.BinOccupancy[i];
	}
	
	// Combine bin triangles
	for (int32 i = 0; i < A.Bins.Num(); i++)
	{
		A.Bins[i].Triangles.Append(B.Bins[i].Triangles);
	}
}

static FRasterizedUV CreateRUV(int32 Width)
{
	check(Width % FRasterizedUV::BinPixelSize == 0);
	int32 BinWidth = Width / FRasterizedUV::BinPixelSize;

	// Create RUV
	FRasterizedUV RUV;
	RUV.Width = Width;
	RUV.Height = Width;
	RUV.Bins.SetNum(BinWidth * BinWidth);
	RUV.BinOccupancy.SetNumZeroed(BinWidth * BinWidth);
	return RUV;
}

VEUV::FTraversalInfo VEUV::FCharting::TraverseLowestCut(const FVoxelGrid& Grid, FChartTraversalContext& Charting)
{
	struct FCandidate
	{
		FTraversalInfo Info;
		float CutValue = 0.0f;
	};
	
	TArray<FCandidate> Candidates;

	// Traverse from visited
	for (int32 VisitedIdx : Charting.Visited)
	{
		const VEUV::FVoxelData& FromVoxel = Grid.Voxels[VisitedIdx];

		// Traverse in all valid directions
		for (const Eigen::Vector3i& Dir : GChartDirections)
		{
			Eigen::Vector3i Pos = FromVoxel.ID + Dir;

			// Bounds check
			if ((Pos.array() < 0).any() ||
				Pos.x() >= Grid.VoxelCount.X ||
				Pos.y() >= Grid.VoxelCount.Y ||
				Pos.z() >= Grid.VoxelCount.Z)
			{
				continue;
			}

			// Find position, if already visited just ignore
			const int32* PosIdx = Grid.VoxelIndexMap.Find(FIntVector(Pos.x(), Pos.y(), Pos.z()));
			if (!PosIdx || Charting.Visited.Contains(*PosIdx))
			{
				continue;
			}
			
			// Skipped or invalid?
			if (Charting.Skipped.Contains(*PosIdx) || Grid.Voxels[*PosIdx].Samples.IsEmpty())
			{
				Charting.Skipped.Add(*PosIdx);
				continue;
			}

			const VEUV::FVoxelData& ToVoxel = Grid.Voxels[*PosIdx];
			
			// Summarize normals on this edge
			Eigen::Vector3f AvgVoxelEdgeNormal = (FromVoxel.SampleAverageNormal + ToVoxel.SampleAverageNormal).normalized();
			
			// Determine cut value, used to prioritize "mostly" continuous traversal
			// Same cut value as the paper, but I feel that this can be improved significantly, same with this loop
			float CutValue = std::abs(AvgVoxelEdgeNormal.dot(Dir.cast<float>())) - FromVoxel.SampleAverageNormal.dot(ToVoxel.SampleAverageNormal);

			// Create candidate
			FCandidate& Candidate = Candidates.AddDefaulted_GetRef();
			Candidate.Info.From = VisitedIdx;
			Candidate.Info.To = *PosIdx;
			Candidate.CutValue = CutValue;
		}
	}
	
	// None found?
	if (Candidates.IsEmpty())
	{
		return {};
	}

	// Sort by cut
	Candidates.Sort([](const FCandidate& A, const FCandidate& B)
	{
		return A.CutValue < B.CutValue;
	});
	
	return Candidates[0].Info;
}

int32 VEUV::FCharting::FirstUnvisited(const FVoxelGrid& Grid, FChartTraversalContext& Charting)
{
	for (int32 i = 0; i < Grid.Voxels.Num(); i++)
	{
		// Ignore empty voxels
		if (Grid.Voxels[i].Samples.IsEmpty())
		{
			continue;
		}
		
		// Skip if already handled
		if (Charting.Visited.Contains(i) || Charting.Skipped.Contains(i))
		{
			continue;
		}
		
		
		return i;
	}
	
	return INDEX_NONE;
}

static void FloodFillSingleChart(
	VEUV::FVoxelGrid& Grid,
	FRasterizedChartTraversalContext& Charting,
	VEUV::FGlobalSolveContext& Solve,
	const VEUV::FUVNormalization& Norm,
	bool bShouldRasterize)
{
	// Create a RUV for the entire chart
	FRasterizedUV ChartRUV;
	if (bShouldRasterize)
	{
		ChartRUV = CreateRUV(Grid.Config.Charting.RasterizationWidth);
	}

	// Find the seed voxel from which we traverse
	int32 SeedIndex = INDEX_NONE;
	for (int32 I = 0; I < Grid.Voxels.Num(); I++)
	{
		if (!Charting.Skipped.Contains(I) && !Charting.Visited.Contains(I))
		{
			SeedIndex = I;
			break;
		}
	}
	check(SeedIndex != INDEX_NONE);

	// Add the seed voxel to the chart
	VEUV::FChartContext& Chart = Solve.Charts.AddDefaulted_GetRef();
	Chart.SolvedUV = Solve.SolvedUV;
	Chart.VoxelIndices.Add(SeedIndex);
	Solve.VoxelChartIndices[SeedIndex] = Solve.Charts.Num() - 1;
	Charting.Visited.Add(SeedIndex);
	Grid.Voxels[SeedIndex].bVisited = true;

	// Summarized UV regions
	TArray<FUVRegion> Regions;
	Regions.Add(GetRegion(Grid, Solve, Grid.Voxels[SeedIndex], Norm));
	
	// Combine the seed RUV with this chart
	if (bShouldRasterize)
	{
		CombineRUV(ChartRUV, Charting.VoxelRUVs[SeedIndex]);
	}
	
	// TODO: This is mostly from the prototype code, I need to refactor this...
	TSet<int32> SkippedSnaoshot = Charting.Skipped;

	// Keep filling until we find no relevant edge
	for (;;)
	{
		// Find the next edge
		VEUV::FTraversalInfo Info = VEUV::FCharting::TraverseLowestCut(Grid, Charting);
		if (Info.From == INDEX_NONE)
		{
			break;
		}

		VEUV::FVoxelData& Voxel = Grid.Voxels[Info.To];
		
		// Get its UV region
		FUVRegion Region = GetRegion(Grid, Solve, Voxel, Norm);

		bool bAnyOverlap = false;
		
		if (Grid.Config.Charting.bEnableOverlapChecks)
		{
			// First, check the UV regions
			for (const FUVRegion& Other : Regions)
			{
				if (RegionIntersection(Region, Other, Grid.Config.Charting.UVRectOverlapEpsilon))
				{
					bAnyOverlap = true;
					break;
				}
			}

			// If any region overlaps, check against its RUV
			if (bAnyOverlap && Grid.Config.Charting.bEnableRasterization)
			{
				bAnyOverlap = RasterizationCheckWithBinTest(ChartRUV, Charting.VoxelRUVs[Info.To]);
			}
		}

		// If overlapping, skip this voxel
		if (bAnyOverlap)
		{
			Voxel.bVisited = false;
			Charting.Skipped.Add(Info.To);
			continue;
		}

		// Combine the RUV with the chart
		if (bShouldRasterize)
		{
			CombineRUV(ChartRUV, Charting.VoxelRUVs[Info.To]);
		}
		
		// Add region
		Regions.Add(Region);

		// Add voxel to chart
		Chart.VoxelIndices.Add(Info.To);
		Voxel.bVisited = true;
		
		// Chart association
		Solve.VoxelChartIndices[Info.To] = Solve.Charts.Num() - 1;
		
		// Mark as visited
		Charting.Visited.Add(Info.To);
	}

	// Reconstruct skipped for successive charting
	Charting.Skipped = SkippedSnaoshot;
	
	// No charting may use the current charted voxels
	for (int32 V : Charting.Visited)
	{
		Charting.Skipped.Add(V);
	}
	
	Charting.Visited.Reset();
}

static void SummarizeRUV(
	const VEUV::FVoxelGrid& Grid,
	const FRasterizedChartTraversalContext& Context,
	const TArray<Eigen::Vector3i>& Faces,
	Eigen::Vector2f& OutMin,
	Eigen::Vector2f& OutMax)
{
	OutMin = Eigen::Vector2f::Constant(FLT_MAX);
	OutMax = Eigen::Vector2f::Constant(-FLT_MAX);

	for (int32 VoxelIndex = 0; VoxelIndex < Grid.Voxels.Num(); VoxelIndex++)
	{
		const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		
		for (int32 FaceIndex : Voxel.Faces)
		{
			const Eigen::Vector3i& Face = Faces[FaceIndex];
			
			Eigen::Vector2f UV0 = Context.CachedVertexUVs[Face.x()];
			Eigen::Vector2f UV1 = Context.CachedVertexUVs[Face.y()];
			Eigen::Vector2f UV2 = Context.CachedVertexUVs[Face.z()];
			
			OutMin = OutMin.cwiseMin(UV0).cwiseMin(UV1).cwiseMin(UV2);
			OutMax = OutMax.cwiseMax(UV0).cwiseMax(UV1).cwiseMax(UV2);
		}
	}
}

void VEUV::FCharting::FillCharts(
	FVoxelGrid& Grid,
	FGlobalSolveContext& SolveContext,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces,
	const TArray<int32>& VertexVoxelIndices)
{
	const bool bShouldRasterize = Grid.Config.Charting.bEnableOverlapChecks && Grid.Config.Charting.bEnableRasterization;

	// We need to rasterize in normalized space, so get the ranges
	FUVNormalization Norm = FSolver::GetNormalization(Grid, SolveContext);
	
	FRasterizedChartTraversalContext Context;
	Context.VoxelRUVs.SetNum(Grid.Voxels.Num());
	
	// Build UV cache
	Context.CachedVertexUVs.SetNumUninitialized(Vertices.Num());
	ParallelFor(Vertices.Num(), [&](int32 VertexIndex)
	{
		if (int32 VoxelIdx = VertexVoxelIndices[VertexIndex]; VoxelIdx != INDEX_NONE)
		{
			Context.CachedVertexUVs[VertexIndex] = Norm.Transform(Grid.SampleUV(SolveContext.SolvedUV, VoxelIdx, Vertices[VertexIndex]));
		}
		else
		{
			Context.CachedVertexUVs[VertexIndex] = Eigen::Vector2f::Zero();
		}
	});

	// Reset visitation state
	for (FVoxelData& Voxel : Grid.Voxels)
	{
		Voxel.bVisited = false;
	}

	// Reset chart indices
	SolveContext.VoxelChartIndices.SetNumUninitialized(Grid.Voxels.Num());
	for (int32& ChartIndex : SolveContext.VoxelChartIndices)
	{
		ChartIndex = INDEX_NONE;
	}

	// Summarize the full RUV range
	if (bShouldRasterize)
	{
		SummarizeRUV(Grid, Context, Faces, Context.RUVMin, Context.RUVMax);
	}

	// Initial voxel states
	for (int32 i = 0; i < Grid.Voxels.Num(); i++)
	{
		const FVoxelData& Voxel = Grid.Voxels[i];

		// If no samples, just skip it entirely
		if (Voxel.Samples.IsEmpty())
		{
			Context.Skipped.Add(i);
		}
	}
	
	// Rasterize all voxels
	if (bShouldRasterize)
	{
		ParallelFor(Grid.Voxels.Num(), [&](int32 VoxelIndex)
		{
			const FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
			if (!Voxel.Samples.IsEmpty())
			{
				Context.VoxelRUVs[VoxelIndex] = CreateRUV(Grid.Config.Charting.RasterizationWidth);
				RasterizeSamples(Context, Voxel, Faces, Context.VoxelRUVs[VoxelIndex]);
			}
		});
	}

	// Keep charting until we're out of voxels
	while (Context.Visited.Num() + Context.Skipped.Num() < Grid.Voxels.Num())
	{
		FloodFillSingleChart(Grid, Context, SolveContext, Norm, bShouldRasterize);
	}
}
