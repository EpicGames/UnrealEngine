// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVEigen.h"
#include "VEUVSolver.h"

namespace VEUV
{
	struct FAABounds
	{
		Eigen::Vector2f Min = Eigen::Vector2f::Constant(FLT_MAX);
		Eigen::Vector2f Max = Eigen::Vector2f::Constant(-FLT_MAX);
	};
	
	struct FEVBounds
	{
		Eigen::VectorXf Mean = Eigen::Vector2f::Zero();
		Eigen::VectorXf EV0 = Eigen::Vector2f::Zero();
		Eigen::VectorXf EV1 = Eigen::Vector2f::Zero();
	};

	struct FChartContext : FSolveRegion
	{
		FAABounds AABounds;
		FAABounds PackedAABounds;

		FEVBounds EVBounds;
		FEVBounds PackedEVBounds;
		
		float Area = 0.0f;
	};

	struct FGlobalSolveContext : FSolveRegion
	{
		TSet<int32> SolvedCornerIndices;
		
		TArray<FChartContext> Charts;
		
		TArray<int32> VoxelChartIndices;

		bool bPackingSucceeded = true;
	};

	struct FChartTraversalContext
	{
		TSet<int32> Visited;
		TSet<int32> Skipped;
	};

	struct FTraversalInfo
	{
		int32 From = INDEX_NONE;
		int32 To   = INDEX_NONE;
	};

	class FCharting
	{
	public:
		/**
		 * Creates separate charts for folded/overlapping UVs
		 * @param Grid Parent grid
		 * @param SolveContext Solution to chart for
		 * @param Vertices Mesh vertices, must be cut to the voxel grid
		 * @param Faces Mesh faces, must be cut to the voxel grid
		 * @param VertexVoxelIndices All vertex voxel associations
		 */
		static void FillCharts(
			FVoxelGrid& Grid,
			FGlobalSolveContext& SolveContext,
			const TArray<Eigen::Vector3f>& Vertices,
			const TArray<Eigen::Vector3i>& Faces,
			const TArray<int32>& VertexVoxelIndices
		);

		/**
		 * Get the first voxel that has the lowest cut to the current visitation context
		 * @param Grid Parent grid
		 * @param Charting Current context
		 * @return {, INDEX_NONE} if none available
		 */
		static FTraversalInfo TraverseLowestCut(
			const FVoxelGrid& Grid,
			FChartTraversalContext& Charting
		);

		/**
		 * Get the first unvisited voxel
		 * @param Grid Parent grid
		 * @param Charting Current context
		 * @return INDEX_NONE if none available
		 */
		static int32 FirstUnvisited(
			const FVoxelGrid& Grid,
			FChartTraversalContext& Charting
		);
	};
}
