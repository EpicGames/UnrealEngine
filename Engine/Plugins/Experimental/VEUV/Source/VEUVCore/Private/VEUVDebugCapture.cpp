// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUV/VEUVDebugCapture.h"

VEUV::FDebugHistory& VEUV::FDebugHistory::Get()
{
	static FDebugHistory Instance;
	return Instance;
}
