// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START

#define EIGEN_NO_DEBUG  1

#include <Eigen/Core>
#include <Eigen/Dense>
#include <Eigen/Eigenvalues>
#include <Eigen/Sparse>

THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
