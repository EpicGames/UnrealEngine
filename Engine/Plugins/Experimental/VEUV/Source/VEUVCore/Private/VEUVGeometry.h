// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVEigen.h"

namespace VEUV
{
	struct FEigenMesh
	{
		TArray<Eigen::Vector3f> Vertices;
		TArray<Eigen::Vector3i> Faces;
	};
}

namespace VEUV::Geometry
{
	inline float Square(float X)
	{
		return X * X;
	}

	inline float WEdge(const Eigen::Vector2f& A, const Eigen::Vector2f& B)
	{
		return A.x() * B.y() - A.y() * B.x();
	}

	inline bool AABBTest(const Eigen::Vector3f& MinA, const Eigen::Vector3f& MaxA, const Eigen::Vector3f& MinB, const Eigen::Vector3f& MaxB)
	{
		return
			(MinA.x() <= MaxB.x() && MaxA.x() >= MinB.x()) &&
			(MinA.y() <= MaxB.y() && MaxA.y() >= MinB.y()) &&
			(MinA.z() <= MaxB.z() && MaxA.z() >= MinB.z());
	}

	inline bool AABBPlaneTest(const Eigen::Vector3f& Min, const Eigen::Vector3f& Max, const Eigen::Vector3f& Normal, float D)
	{
		Eigen::Vector3f VMin, VMax;
		for (int32 I = 0; I < 3; I++)
		{
			if (Normal[I] > 0)
			{
				VMin[I] = Min[I];
				VMax[I] = Max[I];
			}
			else
			{
				VMin[I] = Max[I];
				VMax[I] = Min[I];
			}
		}
		return (VMin.dot(Normal) - D <= 0) && (VMax.dot(Normal) - D >= 0);
	}

	inline bool TriBoxTest(
		const Eigen::Vector3f& BoxMin, const Eigen::Vector3f& BoxMax,
		const Eigen::Vector3f& P0, const Eigen::Vector3f& P1, const Eigen::Vector3f& P2,
		float PlaneEpsilon = 0.0f)
	{
        // Tomas Möller
		
		Eigen::Vector3f MinTri = P0.cwiseMin(P1).cwiseMin(P2);
		Eigen::Vector3f MaxTri = P0.cwiseMax(P1).cwiseMax(P2);

		Eigen::Vector3f Min = BoxMin - Eigen::Vector3f::Constant(PlaneEpsilon);
		Eigen::Vector3f Max = BoxMax + Eigen::Vector3f::Constant(PlaneEpsilon);

		if (!AABBTest(Min, Max, MinTri, MaxTri))
		{
			return false;
		}

		Eigen::Vector3f Edges[] = {
			P1 - P0, 
			P2 - P0, 
			P0 - P2
		};
		
		Eigen::Vector3f N = Edges[0].cross(Edges[1]);
		float D = N.dot(P0);

		if (!AABBPlaneTest(Min, Max, N, D))
		{
			return false;
		}

		Eigen::Vector3f C = (Max + Min) * 0.5f;
		Eigen::Vector3f H = (Max - Min) * 0.5f;

		Eigen::Vector3f V0 = P0 - C;
		Eigen::Vector3f V1 = P1 - C;
		Eigen::Vector3f V2 = P2 - C;

		Eigen::Vector3f E[] = {
			Eigen::Vector3f(1,0,0),
			Eigen::Vector3f(0,1,0),
			Eigen::Vector3f(0,0,1)
		};
		
		Eigen::Vector3f F[] = {
			V1 - V0,
			V2 - V1,
			V0 - V2
		};

		for (int32 I = 0; I < 3; I++)
		{
			for (int32 J = 0; J < 3; J++)
			{
				Eigen::Vector3f A = E[I].cross(F[J]);
				float PA0 = A.dot(V0), PA1 = A.dot(V1), PA2 = A.dot(V2);
				float MinP = std::min({PA0, PA1, PA2});
				float MaxP = std::max({PA0, PA1, PA2});
				float R = A.cwiseAbs().dot(H);
				if (MinP > R || MaxP < -R)
				{
					return false;
				}
			}
		}
		return true;
	}

	inline bool PointInTri(const Eigen::Vector2f& P, const Eigen::Vector2f& A, const Eigen::Vector2f& B, const Eigen::Vector2f& C, float BoundaryEps = 0.0f)
	{
		Eigen::Vector2f EAB = B - A;
		Eigen::Vector2f EAC = C - A;
		Eigen::Vector2f P2 = P - A;

		float D = EAC.squaredNorm() * EAB.squaredNorm() - Square(EAB.dot(EAC));
		float U = (EAB.squaredNorm() * P2.dot(EAC) - EAB.dot(EAC) * P2.dot(EAB)) / D;
		float V = (EAC.squaredNorm() * P2.dot(EAB) - EAB.dot(EAC) * P2.dot(EAC)) / D;

		return U >= BoundaryEps && V >= BoundaryEps && U + V <= 1.0f - BoundaryEps * 2.0f;
	}

	inline bool LineLineTest(const Eigen::Vector2f& A0, const Eigen::Vector2f& A1, const Eigen::Vector2f& B0, const Eigen::Vector2f& B1, float BoundaryEps = 1e-3f)
	{
		constexpr float Eps = 1e-9f;

		float D = (B1.y() - B0.y()) * (A1.x() - A0.x()) - (B1.x() - B0.x()) * (A1.y() - A0.y());
		if (std::abs(D) < Eps)
		{
			if (std::abs(WEdge(B0 - A0, A1 - A0)) < Eps)
			{
				Eigen::Vector2f AMin = A0.cwiseMin(A1), AMax = A0.cwiseMax(A1);
				Eigen::Vector2f BMin = B0.cwiseMin(B1), BMax = B0.cwiseMax(B1);
				if (AMax.x() - AMin.x() > Eps)
				{
					return std::max(AMin.x(), BMin.x()) + BoundaryEps <= std::min(AMax.x(), BMax.x()) - BoundaryEps;
				}
				else
				{
					return std::max(AMin.y(), BMin.y()) + BoundaryEps <= std::min(AMax.y(), BMax.y()) - BoundaryEps;
				}
			}
			return false;
		}

		float U = ((B1.x() - B0.x()) * (A0.y() - B0.y()) - (B1.y() - B0.y()) * (A0.x() - B0.x())) / D;
		float V = ((A1.x() - A0.x()) * (A0.y() - B0.y()) - (A1.y() - A0.y()) * (A0.x() - B0.x())) / D;
		return (U >= BoundaryEps && U <= 1.0f - BoundaryEps) && (V >= BoundaryEps && V <= 1.0f - BoundaryEps);
	}

	inline bool LineTriTestOuter(const Eigen::Vector2f& A0, const Eigen::Vector2f& A1, const Eigen::Vector2f& B0, const Eigen::Vector2f& B1, const Eigen::Vector2f& B2, float BoundaryEps)
	{
		return LineLineTest(A0, A1, B0, B1, BoundaryEps) ||
			   LineLineTest(A0, A1, B1, B2, BoundaryEps) ||
			   LineLineTest(A0, A1, B2, B0, BoundaryEps);
	}

	inline bool TriTriTest(
		const Eigen::Vector2f& A0, const Eigen::Vector2f& A1, const Eigen::Vector2f& A2,
		const Eigen::Vector2f& B0, const Eigen::Vector2f& B1, const Eigen::Vector2f& B2,
		float BoundaryEps = 0.1f)
	{
		if (LineTriTestOuter(A0, A1, B0, B1, B2, BoundaryEps) ||
			LineTriTestOuter(A1, A2, B0, B1, B2, BoundaryEps) ||
			LineTriTestOuter(A2, A0, B0, B1, B2, BoundaryEps))
		{
			return true;
		}

		if (PointInTri(A0, B0, B1, B2, BoundaryEps) || PointInTri(B0, A0, A1, A2, BoundaryEps))
		{
			return true;
		}

		return false;
	}

	template<typename FuncType>
	inline void IntersectSegmentPlane(const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& N, float D, FuncType&& Func)
	{
		float Num = D - N.dot(A);
		float Den = (B - A).dot(N);

		if (std::abs(Den) < 1e-6f)
		{
			if (std::abs(Num) < 1e-6f)
			{
				Func(A);
				Func(B);
			}
			return;
		}

		float T = Num / Den;
		if (T >= 0.0f && T <= 1.0f)
		{
			Func(A + (B - A) * T);
		}
	}

	template<typename FuncType>
	inline void IntersectTrianglePlane(const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& C, const Eigen::Vector3f& N, float D, FuncType&& Func)
	{
		IntersectSegmentPlane(A, B, N, D, Func);
		IntersectSegmentPlane(B, C, N, D, Func);
		IntersectSegmentPlane(A, C, N, D, Func);
	}

	inline bool ConditionalVoxelPointAdd(const Eigen::Vector3f& A, const Eigen::Vector3f& Center, const Eigen::Vector3f& Extent, float Eps = 1e-4f)
	{
		return ((A - Center).cwiseAbs().array() <= (Extent.array() + Eps)).all();
	}
	
	inline void TriangulatePoints(
		const Eigen::Vector3f& TriA, const Eigen::Vector3f& TriB, const Eigen::Vector3f& TriC,
		TArray<Eigen::Vector3f>& Points,
		TArray<Eigen::Vector3f>& OutVertices,
		TArray<Eigen::Vector3f>& OutNormals,
		TArray<Eigen::Vector3i>& OutFaces)
	{
		if (Points.Num() < 3)
		{
			return;
		}

		Eigen::Vector3f E0 = (TriB - TriA).normalized();
		Eigen::Vector3f E2 = (TriC - TriA).normalized();
		Eigen::Vector3f N = E0.cross(E2).normalized();

		Eigen::Vector3f B0 = (TriC - TriB).normalized();
		Eigen::Vector3f B1 = N.cross(B0).normalized();

		Eigen::Vector2f Center = Eigen::Vector2f::Zero();
		for (const Eigen::Vector3f& Point : Points)
		{
			Center += Eigen::Vector2f(B0.dot(Point), B1.dot(Point));
		}
		Center /= static_cast<float>(Points.Num());

		Points.Sort([&](const Eigen::Vector3f& PA, const Eigen::Vector3f& PB)
		{
			Eigen::Vector2f AP(B0.dot(PA), B1.dot(PA));
			Eigen::Vector2f BP(B0.dot(PB), B1.dot(PB));
			return std::atan2(AP.y() - Center.y(), AP.x() - Center.x())
				 < std::atan2(BP.y() - Center.y(), BP.x() - Center.x());
		});

		int32 IndexOffset = OutVertices.Num();
		for (const Eigen::Vector3f& Point : Points)
		{
			OutVertices.Add(Point);
			OutNormals.Add(N);
		}

		for (int32 i = 1; i < Points.Num() - 1; i++)
		{
			OutFaces.Add(Eigen::Vector3i(IndexOffset, IndexOffset + i, IndexOffset + i + 1));
		}
	}

	inline void ClipTriangleToVoxel(
		const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& C,
		const Eigen::Vector3f& Min, const Eigen::Vector3f& Max,
		TArray<Eigen::Vector3f>& OutPoints)
	{
		// Sutherland-Hodgman, not particularly optimized
		
		TArray<Eigen::Vector3f, TInlineAllocator<16>> Polygon;
		Polygon.Add(A);
		Polygon.Add(B);
		Polygon.Add(C);

		TArray<Eigen::Vector3f, TInlineAllocator<16>> WorkingPolygon;

		for (int32 Axis = 0; Axis < 3; Axis++)
		{
			for (int32 Side = 0; Side < 2; Side++)
			{
				if (Polygon.IsEmpty())
				{
					break;
				}

				float Sign     = (Side == 0) ? 1.0f : -1.0f;
				float Boundary = (Side == 0) ? Max[Axis] : -Min[Axis];

				WorkingPolygon.Reset();

				for (int32 VertexIndex = 0; VertexIndex < Polygon.Num(); VertexIndex++)
				{
					const Eigen::Vector3f& Current = Polygon[VertexIndex];
					const Eigen::Vector3f& Next    = Polygon[(VertexIndex + 1) % Polygon.Num()];

					float Dist     = Sign * Current[Axis] - Boundary;
					float DistNext = Sign * Next[Axis]    - Boundary;

					if (Dist <= 0.0f)
					{
						WorkingPolygon.Add(Current);
					}

					if (Dist * DistNext < 0.0f)
					{
						// Sort endpoints by their cut-axis coordinate before lerping,
						// so result matches regardless of edge direction
						const Eigen::Vector3f* From = &Current;
						const Eigen::Vector3f* To = &Next;
						float DFrom = Dist;
						float DTo = DistNext;
						if ((*To)[Axis] < (*From)[Axis])
						{
							Swap(From, To);
							Swap(DFrom, DTo);
						}
						float T = DFrom / (DFrom - DTo);

						// Snap a near-endpoint intersection back onto the endpoint itself
						constexpr float SnapEps = 1e-5f;
						if (T < SnapEps)
						{
							WorkingPolygon.Add(*From);
						}
						else if (T > 1.0f - SnapEps)
						{
							WorkingPolygon.Add(*To);
						}
						else
						{
							WorkingPolygon.Add(*From + (*To - *From) * T);
						}
					}
				}

				Swap(Polygon, WorkingPolygon);
			}
		}

		OutPoints = MoveTemp(Polygon);
	}
	
	inline void GetTrilinearWeights(float X, float Y, float Z, float OutW[8])
	{
		OutW[0] = (1-X)*(1-Y)*(1-Z); 
		OutW[1] = (1-X)*(1-Y)*Z;
		OutW[2] = (1-X)*Y*(1-Z);
		OutW[3] = (1-X)*Y*Z;
		OutW[4] = X*(1-Y)*(1-Z);
		OutW[5] = X*(1-Y)*Z;
		OutW[6] = X*Y*(1-Z);
		OutW[7] = X*Y*Z;
	}

	inline void GetTrilinearPartialXYZDerivatives(float X, float Y, float Z, Eigen::Vector3f OutPD[8])
	{
		OutPD[0] = Eigen::Vector3f(-(1-Y)*(1-Z), -(1-X)*(1-Z), -(1-X)*(1-Y));
		OutPD[1] = Eigen::Vector3f(-(1-Y)*Z,     -(1-X)*Z,      (1-X)*(1-Y));
		OutPD[2] = Eigen::Vector3f(-Y*(1-Z),      (1-X)*(1-Z), -(1-X)*Y);
		OutPD[3] = Eigen::Vector3f(-Y*Z,          (1-X)*Z,      (1-X)*Y);
		OutPD[4] = Eigen::Vector3f( (1-Y)*(1-Z), -X*(1-Z),     -X*(1-Y));
		OutPD[5] = Eigen::Vector3f( (1-Y)*Z,     -X*Z,          X*(1-Y));
		OutPD[6] = Eigen::Vector3f( Y*(1-Z),      X*(1-Z),     -X*Y);
		OutPD[7] = Eigen::Vector3f( Y*Z,          X*Z,           X*Y);
	}
	
	inline Eigen::Vector3f SafeOrthogonalize(
		const Eigen::Vector3f& T1,
		const Eigen::Vector3f& V,
		const Eigen::Vector3f& Normal)
	{
		Eigen::Vector3f Residual = V - T1 * V.dot(T1);
		float Len = Residual.norm();
		return (Len > 1e-6f) ? Residual / Len : Normal.cross(T1).normalized();
	}

	inline Eigen::Vector3f GetBarycentrics(const Eigen::Vector3f& P, const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& C)
	{
		Eigen::Vector3f EAB = B - A;
		Eigen::Vector3f EAC = C - A;
		Eigen::Vector3f P2  = P - A;

		float DAB = EAB.dot(EAB);
		float DAC = EAB.dot(EAC);
		float DCC = EAC.dot(EAC);
		float DPB = P2.dot(EAB);
		float DPC = P2.dot(EAC);
		float Denom = DAB * DCC - DAC * DAC;

		// Denom/(DAB*DCC) = sin^2(theta) between the edges, so this threshold is scale-invariant
		const float NormSq = DAB * DCC;
		if (NormSq <= 0.0f || Denom < 1e-7f * NormSq)
		{
			return Eigen::Vector3f(1.0f, 0.0f, 0.0f);
		}

		float V = (DCC * DPB - DAC * DPC) / Denom;
		float W = (DAB * DPC - DAC * DPB) / Denom;
		float U = 1.0f - V - W;
		return Eigen::Vector3f(U, V, W);
	}
}
