// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUV/VEUVOptimizer.h"
#include "VEUV/VEUVDebugCapture.h"
#include "VEUVVoxelGrid.h"
#include "VEUVSolver.h"
#include "VEUVCharting.h"
#include "VEUVPacking.h"
#include "VEUVSplitMesh.h"
#include "VEUVSampling.h"
#include "VEUVSchedulers.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Async/ParallelFor.h"

static TAutoConsoleVariable<int32> CVarVEUVDebugCaptureEnabled(
	TEXT("VEUV.Debug.CaptureEnabled"),
	0,
	TEXT("When enabled, VEUV builds store debug snapshots for the visualizer panel"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarVEUVDebugSnapshotsEnabled(
	TEXT("VEUV.Debug.SnapshotsEnabled"),
	0,
	TEXT("When enabled, VEUV captures per-iteration UV snapshots for playback in the debug panel"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarVEUVProfileIterations(
	TEXT("VEUV.Profile.Iterations"),
	1,
	TEXT("Number of times to run VEUV generation for profiling"),
	ECVF_Default
);

struct FVEUVTimer
{
	FVEUVTimer() : Start(FPlatformTime::Seconds()), Last(Start)
	{
		
	}

	/** Mark a time stamp since last */
	void Mark(double& OutMs)
	{
		double Now = FPlatformTime::Seconds();
		OutMs = (Now - Last) * 1000.0;
		Last = Now;
	}

	/** Ignore the most recent accumulated time */
	void Ignore()
	{
		double Now = FPlatformTime::Seconds();
		Ignored += (Now - Last);
		Last = FPlatformTime::Seconds();
	}

	/** Get the total time elapsed */
	void Total(double& OutMs) const
	{
		OutMs = (FPlatformTime::Seconds() - Start - Ignored) * 1000.0;
	}

	double Ignored = 0;
	double Start;
	double Last;
};

static TArray<Eigen::Vector3f> ToEigen(const TArray<FVector3f>& In)
{
	TArray<Eigen::Vector3f> Out;
	Out.SetNumUninitialized(In.Num());
	
	for (int32 I = 0; I < In.Num(); I++)
	{
		Out[I] = Eigen::Vector3f(In[I].X, In[I].Y, In[I].Z);
	}
	
	return Out;
}

static TArray<Eigen::Vector3i> ToEigen(const TArray<FIntVector>& In)
{
	TArray<Eigen::Vector3i> Out;
	Out.SetNumUninitialized(In.Num());
	
	for (int32 I = 0; I < In.Num(); I++)
	{
		Out[I] = Eigen::Vector3i(In[I].X, In[I].Y, In[I].Z);
	}
	
	return Out;
}

static TArray<FVector3f> FromEigen(const TArray<Eigen::Vector3f>& In)
{
	TArray<FVector3f> Out;
	Out.SetNumUninitialized(In.Num());
	
	for (int32 I = 0; I < In.Num(); I++)
	{
		Out[I] = FVector3f(In[I].x(), In[I].y(), In[I].z());
	}
	
	return Out;
}

static TArray<FIntVector> FromEigen(const TArray<Eigen::Vector3i>& In)
{
	TArray<FIntVector> Out;
	Out.SetNumUninitialized(In.Num());
	
	for (int32 I = 0; I < In.Num(); I++)
	{
		Out[I] = FIntVector(In[I].x(), In[I].y(), In[I].z());
	}
	
	return Out;
}

static void AssignVertexUVs(
	const FVEUVConfig& Config,
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FGlobalSolveContext& SolveContext,
	const VEUV::FEigenMesh& SplitMesh,
	const VEUV::FEigenMesh& SeamSplitMesh,
	const TArray<int32>& SplitVertexVoxelIndices,
	TArray<FVector2f>& VertexUVs)
{
	VertexUVs.SetNumZeroed(SeamSplitMesh.Vertices.Num());
	
	// Standard assignment mode
	if (Config.UVVisualizeMode == EVEUVVisualizationDebugMode::None)
	{
		ParallelFor(SeamSplitMesh.Vertices.Num(), [&](int32 i)
		{
			// VoxelIdx is the voxel that contains the vertex position, so the trilinear cage interpolates rather than extrapolates.
			int32 VoxelIdx = SplitVertexVoxelIndices.IsValidIndex(i) ? SplitVertexVoxelIndices[i] : INDEX_NONE;
			if (VoxelIdx == INDEX_NONE)
			{
				return;
			}

			// Get chart from voxel
			int32 ChartIdx = SolveContext.VoxelChartIndices.IsValidIndex(VoxelIdx) ? SolveContext.VoxelChartIndices[VoxelIdx] : INDEX_NONE;
			if (ChartIdx == INDEX_NONE)
			{
				return;
			}

			// Sample the UV
			Eigen::Vector2f UV = Grid.SampleUV(SolveContext.Charts[ChartIdx].SolvedUV, VoxelIdx, SeamSplitMesh.Vertices[i]);
			VertexUVs[i] = FVector2f(UV.x(), UV.y());
		});
		
		return;
	}

	// Helper, assigns weights
	auto DebugWeightAssignment = [&](auto WeightGetter)
	{
		float MaxW = 0.0f;
		for (const VEUV::FVoxelData& Voxel : Grid.Voxels)
		{
			for (int32 i = 0; i < Voxel.Faces.Num(); i++)
			{
				MaxW = FMath::Max(MaxW, WeightGetter(Voxel, i));
			}
		}

		if (MaxW < 1e-10f)
		{
			return;
		}

		// Faces guaranteed to be from voxel split mesh, so unique association
		for (const VEUV::FVoxelData& Voxel : Grid.Voxels)
		{
			for (int32 i = 0; i < Voxel.Faces.Num(); i++)
			{
				const Eigen::Vector3i& Face = SplitMesh.Faces[Voxel.Faces[i]];
				FVector2f UV(WeightGetter(Voxel, i) / MaxW, 0);
				VertexUVs[Face.x()] = UV;
				VertexUVs[Face.y()] = UV;
				VertexUVs[Face.z()] = UV;
			}
		}
	};

	// Handle mode
	switch (Config.UVVisualizeMode)
	{
		default:
		{
			checkNoEntry();
			break;
		}
		case EVEUVVisualizationDebugMode::P_Area:
		{
			DebugWeightAssignment([](const VEUV::FVoxelData& V, int32 I) { return V.FaceAreaWeights[I]; });
			break;
		}
		case EVEUVVisualizationDebugMode::P_Complexity:
		{
			DebugWeightAssignment([](const VEUV::FVoxelData& V, int32 I) { return V.FaceComplexityWeights[I]; });
			break;
		}
		case EVEUVVisualizationDebugMode::P_Combined:
		{
			DebugWeightAssignment([](const VEUV::FVoxelData& V, int32 I) { return V.FaceAreaWeights[I] + V.FaceComplexityWeights[I]; });
			break;
		}
	}
}

static void CreateStatistics(
	VEUV::FStatistics& Stats,
	const FVEUVConfig& Config,
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FEigenMesh& InputMesh,
	const VEUV::FEigenMesh& OutputMesh,
	const VEUV::FGlobalSolveContext& SolveContext,
	const VEUV::FNormalization& Norm)
{
	// Standard stuff
	Stats.Label = Config.Label;
	Stats.InputVertices = InputMesh.Vertices.Num();
	Stats.InputFaces = InputMesh.Faces.Num();
	Stats.OutputVertices = OutputMesh.Vertices.Num();
	Stats.OutputFaces = OutputMesh.Faces.Num();
	Stats.ChartCount = SolveContext.Charts.Num();
	Stats.VoxelCount = Grid.Voxels.Num();
	Stats.RasterizationWidth = Config.Charting.RasterizationWidth;

	// TODO: Keep a transient thing somewhere, avoid redoing this...
	{
		Eigen::Vector3f Extent = Grid.LogicalMax - Grid.LogicalMin;
		float CellSize = FMath::Max(Config.Sampling.ComplexityFieldCellSize, 1e-6f) / FMath::Max(Norm.Scale, 1e-6f);
		int32 MaxRes = FMath::Max(2, Config.Sampling.ComplexityFieldMaxResolution);
		Stats.ComplexityFieldResolution = FIntVector(
			FMath::Clamp(FMath::CeilToInt32(Extent.x() / CellSize), 2, MaxRes),
			FMath::Clamp(FMath::CeilToInt32(Extent.y() / CellSize), 2, MaxRes),
			FMath::Clamp(FMath::CeilToInt32(Extent.z() / CellSize), 2, MaxRes));
	}

	// Accumulate voxel stats
	for (const VEUV::FVoxelData& Voxel : Grid.Voxels)
	{
		if (!Voxel.Samples.IsEmpty())
		{
			Stats.OccupiedVoxels++;
		}
		
		Stats.TotalSamples += Voxel.Samples.Num();
	}
}

static void ValidateResult(VEUV::FResult& Result, const VEUV::FGlobalSolveContext& SolveContext)
{
	using namespace VEUV;

	// Packing is currently all-or-nothing; flag every chart on failure
	if (!SolveContext.bPackingSucceeded)
	{
		for (FChart& Chart : Result.Charts)
		{
			Chart.Status |= EChartStatus::PackingFailed;
		}
	}

	auto SignedArea = [&Result](int32 FaceIdx) -> float
	{
		const FInt32Vector3& F = Result.OutputMesh.Faces[FaceIdx];
		return 0.5f * FVector2f::CrossProduct(
			Result.VertexUVs[F.Y] - Result.VertexUVs[F.X],
			Result.VertexUVs[F.Z] - Result.VertexUVs[F.X]);
	};

	auto IsFiniteUV = [&Result](int32 FaceIdx) -> bool
	{
		const FInt32Vector3& F = Result.OutputMesh.Faces[FaceIdx];
		// Note ContainsNaN returns true for NaN OR Inf
		return !Result.VertexUVs[F.X].ContainsNaN()
		    && !Result.VertexUVs[F.Y].ContainsNaN()
		    && !Result.VertexUVs[F.Z].ContainsNaN();
	};

	// Compute per-chart signed UV area and unsigned area sums, and set NonFinite flags
	TArray<float> ChartSignedAreaSum;
	ChartSignedAreaSum.SetNumZeroed(Result.Charts.Num());
	TArray<int32> ChartFaceCounts;
	ChartFaceCounts.SetNumZeroed(Result.Charts.Num());

	for (int32 FaceIdx = 0; FaceIdx < Result.OutputMesh.Faces.Num(); FaceIdx++)
	{
		const int32 ChartIdx = Result.FaceChartIndices.IsValidIndex(FaceIdx) ? Result.FaceChartIndices[FaceIdx] : INDEX_NONE;
		if (!Result.Charts.IsValidIndex(ChartIdx))
		{
			Result.Stats.UnassignedFaceCount++;
			continue;
		}

		ChartFaceCounts[ChartIdx]++;

		if (!IsFiniteUV(FaceIdx))
		{
			Result.Charts[ChartIdx].Status |= EChartStatus::NonFinite;
			continue;
		}

		const float Signed = SignedArea(FaceIdx);
		Result.Stats.TotalUVArea     += FMath::Abs(Signed);
		ChartSignedAreaSum[ChartIdx] += Signed;
	}

	// Flag charts with no output faces (whether unvoxelized or lost during meshing)
	for (int32 ChartIdx = 0; ChartIdx < Result.Charts.Num(); ChartIdx++)
	{
		if (ChartFaceCounts[ChartIdx] == 0)
		{
			Result.Charts[ChartIdx].Status |= EChartStatus::Empty;
		}
	}

	// Use signed area sum to count inverted (w.r.t. majority) faces
	for (int32 FaceIdx = 0; FaceIdx < Result.OutputMesh.Faces.Num(); FaceIdx++)
	{
		const int32 ChartIdx = Result.FaceChartIndices.IsValidIndex(FaceIdx) ? Result.FaceChartIndices[FaceIdx] : INDEX_NONE;
		if (!Result.Charts.IsValidIndex(ChartIdx) || !IsFiniteUV(FaceIdx))
		{
			continue;
		}

		// Note a perfectly zero-net-signed-area chart is treated as nominally positive, so half its faces still get flagged
		const float MajoritySign = ChartSignedAreaSum[ChartIdx] >= 0.0f ? 1.0f : -1.0f;
		const float Signed = SignedArea(FaceIdx);
		if (Signed != 0.0f && FMath::Sign(Signed) != MajoritySign)
		{
			Result.Charts[ChartIdx].Status |= EChartStatus::Inverted;
			Result.Stats.InvertedUVArea += FMath::Abs(Signed);
			Result.Stats.InvertedFaceCount++;
		}
	}

	// Aggregate per-chart status into the result-level status
	Result.Status = EChartStatus::Valid;
	Result.Stats.FailedChartCount = 0;
	for (const FChart& Chart : Result.Charts)
	{
		Result.Status |= Chart.Status;
		Result.Stats.FailedChartCount += int32(Chart.Status != EChartStatus::Valid);
	}
}

static void FinalizeDebugCapture(
	const FVEUVConfig& Config,
	const VEUV::FResult& Result,
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FGlobalSolveContext& SolveContext,
	const VEUV::FNormalization& Norm,
	const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	// Fill capture
	Capture->Timestamp = FDateTime::Now();
	Capture->Config = Config;
	Capture->Result = Result;

	for (int32 i = 0; i < Grid.Voxels.Num(); i++)
	{
		const VEUV::FVoxelData& Voxel = Grid.Voxels[i];
		
		// Destination debug data
		VEUV::FDebugCapture::FVoxelDebug& VoxelDebug = Capture->Voxels.Emplace_GetRef();

		// Draw voxel bounds
		if (!Voxel.Samples.IsEmpty())
		{
			VoxelDebug.bOccupied = true;
			
			Eigen::Vector3f WMin = Norm.ToWorld(Voxel.LogicalMin);
			Eigen::Vector3f WMax = Norm.ToWorld(Voxel.LogicalMax);
			FVector TMin = Config.DebugWorldTransform.TransformPosition(FVector(WMin.x(), WMin.y(), WMin.z()));
			FVector TMax = Config.DebugWorldTransform.TransformPosition(FVector(WMax.x(), WMax.y(), WMax.z()));
			VoxelDebug.Center = (TMin + TMax) * 0.5;
			VoxelDebug.Extent = (TMax - TMin).GetAbs() * 0.5;
		}

		// Draw sample positions
		for (const VEUV::FSampleData& Sample : Voxel.Samples)
		{
			Eigen::Vector3f LocalPosition = Norm.ToWorld(Sample.Position);
			
			FVector WorldPosition = Config.DebugWorldTransform.TransformPosition(FVector(LocalPosition.x(), LocalPosition.y(), LocalPosition.z()));
			FVector WorldNormal   = Config.DebugWorldTransform.TransformVectorNoScale(FVector(Sample.Normal.x(), Sample.Normal.y(), Sample.Normal.z()));
			
			VEUV::FDebugCapture::FDebugSample S;
			S.Position = WorldPosition + WorldNormal * 1.0f;
			
			switch (Sample.Type)
			{
			case EVEUVSampleType::None:
				S.Color = FColor::Black;
				S.Type = EVEUVSampleType::None;
				break;
			case EVEUVSampleType::Area:
				S.Color = FColor::White;
				S.Type = EVEUVSampleType::Area;
				break;
			case EVEUVSampleType::Complexity:
				S.Color = FColor::Orange;
				S.Type = EVEUVSampleType::Complexity;
				break;
			case EVEUVSampleType::Adaptive:
				S.Color = FColor::Blue;
				S.Type = EVEUVSampleType::Adaptive;
				break;
			}
			
			Capture->Samples.Add(S);
		}
	}

	// Copy per-chart EV bounds from solve context
	Capture->ChartEVs.SetNum(SolveContext.Charts.Num());
	for (int32 ChartIdx = 0; ChartIdx < SolveContext.Charts.Num(); ChartIdx++)
	{
		const VEUV::FChartContext& Src = SolveContext.Charts[ChartIdx];
		VEUV::FDebugCapture::FChartEVDebug& Dst = Capture->ChartEVs[ChartIdx];
		Dst.Mean = FVector2f(Src.PackedEVBounds.Mean.x(), Src.PackedEVBounds.Mean.y());
		Dst.EV0 = FVector2f(Src.PackedEVBounds.EV0.x(), Src.PackedEVBounds.EV0.y());
		Dst.EV1 = FVector2f(Src.PackedEVBounds.EV1.x(), Src.PackedEVBounds.EV1.y());
	}

	VEUV::FDebugHistory::Get().Add(Capture);
}

VEUV::FResult GenerateInternal(const VEUV::FMesh& InputMesh, const FVEUVConfig& Config)
{
	VEUV::FResult Result;

	TRACE_CPUPROFILER_EVENT_SCOPE(VEUV::Parameterize);
	FVEUVTimer Timer;

	TSharedPtr<VEUV::FDebugCapture> Capture;
	if (CVarVEUVDebugCaptureEnabled->GetBool())
	{
		Capture = MakeShared<VEUV::FDebugCapture>();
		Capture->bWithSnapshots = CVarVEUVDebugSnapshotsEnabled->GetBool();
	}
	
	// Convert the mesh to Eigen typing
	VEUV::FEigenMesh Mesh;
	Mesh.Vertices = ToEigen(InputMesh.Vertices);
	Mesh.Faces    = ToEigen(InputMesh.Faces);

	// Standard solving really does require normalization for well formed problems
	VEUV::FNormalization Norm;
	if (Config.bEnableVertexNormalization)
	{
		Norm = VEUV::FNormalization::Compute(Mesh.Vertices);
	}
	
	// Create the uniform voxel grid
	VEUV::FVoxelGrid Grid;
	Grid.Distribute(Mesh.Vertices, Config);
	Timer.Mark(Result.Stats.DistributeMs);

	// Split the mesh by all voxels, this is not the final mesh representation
	// It makes sampling and other operations much easier to reason for
	VEUV::FEigenMesh SplitMesh;
	TArray<int32> SplitVertexVoxelIndices;
	TArray<VEUV::FVertexSource> SplitVertexSources;
	VEUV::FSplitMesh::SplitByVoxels(Grid, Mesh, SplitMesh, SplitVertexVoxelIndices, SplitVertexSources);
	Timer.Mark(Result.Stats.SplitFacesMs);

	// Create the initial sample set to parameterize against 
	VEUV::FSampling::SampleVoxels(Grid, Config, Norm, SplitMesh.Vertices, SplitMesh.Faces);
	Timer.Mark(Result.Stats.SampleVoxelsMs);

	// If no voxel ended up with faces, mark the result as Empty and early-out
	if (Grid.CornerAllocator == 0)
	{
		Result.Status |= VEUV::EChartStatus::Empty;
		return Result;
	}

	// Solve for the continuous field with R78 constraints
	// This aims to create conformality and orthogonality
	VEUV::FGlobalSolveContext SolveContext;
	VEUV::FSolver::SolveContinuousDenseField78(Grid, SolveContext, Capture);
	Timer.Mark(Result.Stats.ContinuousField78Ms);

	// If adaptive is enabled, we re-sample on domains with high error
	if (Config.Sampling.AdaptiveSubdivisions > 0)
	{
		Result.Stats.AdaptiveSamplesAdded = VEUV::FSampling::AdaptiveResample(Grid, Config, Norm, SolveContext, SplitMesh.Vertices, SplitMesh.Faces);
		Timer.Mark(Result.Stats.AdaptiveResampleMs);
	}
	
	// If global solving is enabled, we re-solve against the entire constraint set without any pinned corners with R9ab constraints
	// This is the area preservation part, R78 "may" result in something not too dissimilar, but R9ab actually makes it part of the solve
	VEUV::Schedulers::FAdam::FCoeffInvariantSnapshot AdamSnapshot;
	if (Config.Solver.bEnableGlobalSolve)
	{
		// Pre-condition the solution for injectivity, much easier to optimize for in isolation
		Result.Stats.GlobalField10Iterations = VEUV::FSolver::SolveGlobalFieldR10(
			Grid, SolveContext, SplitMesh,
			Config.Solver.GlobalIterations,
			Capture
		);
		Timer.Mark(Result.Stats.GlobalField10Ms);
		
		// Now that we have a good base, do R9ab, we don't need to force injectivity yet
		Result.Stats.GlobalField9Iterations = VEUV::FSolver::SolveGlobalFieldR9AB(
			Grid, SolveContext, 
			AdamSnapshot, SplitMesh,
			Config.Solver.GlobalIterations,
			true, false, 
			Capture
		);
		Timer.Mark(Result.Stats.GlobalField9Ms);
	}

	// Now that we have a UV field, we need to traverse the voxels and find folds/overlapping
	// On folds, we chart separately
	VEUV::FCharting::FillCharts(Grid, SolveContext, SplitMesh.Vertices, SplitMesh.Faces, SplitVertexVoxelIndices);
	Timer.Mark(Result.Stats.ChartingMs);

	// With the produced charts, re-solve against R9ab
	// This refinement has a much easier time conforming
	if (Config.Solver.bEnableGlobalSolve && Config.Solver.bEnableChartRefinement)
	{
		int32 RefinementIterations = FMath::Max<int32>(1, Config.Solver.GlobalIterations * Config.Solver.ChartRefinementIterationFraction);

		// Refine each chart, given that this is final we now force injectivity
		for (VEUV::FChartContext& Chart : SolveContext.Charts)
		{
			Result.Stats.ChartRefinementIterations += VEUV::FSolver::SolveGlobalFieldR9AB(
				Grid, Chart, 
				AdamSnapshot, SplitMesh,
				RefinementIterations,
				false, Config.Solver.bChartRefinementForceInjectivity,
				Capture
			);
		}

		Timer.Mark(Result.Stats.ChartRefinementMs);
	}

	// Normalize the produced UV field, the solve field couldn't care less about its range
	VEUV::FSolver::Normalize(Grid, SolveContext);
	Timer.Mark(Result.Stats.NormalizeMs);

	// If reorienting, find the EVs of the charts and rotate it to something that's easier to pack down
	if (Config.Packing.bEnableChartReorientation)
	{
		VEUV::FPacking::ReorientCharts(Grid, SolveContext, SplitMesh.Vertices, SplitMesh.Faces);
		Timer.Mark(Result.Stats.ReorientMs);
	}

	// If packing, do a typical scanline layout of the re-oriented charts
	if (Config.Packing.bEnableAtlasPacking)
	{
		VEUV::FPacking::PackCharts(Grid, SolveContext);
		Timer.Mark(Result.Stats.PackMs);
	}
	
	// Optional, re-normalization, ill-formed if packing
	if (Config.Packing.bEnableUVNormalization)
	{
		for (VEUV::FChartContext& Chart : SolveContext.Charts)
		{
			VEUV::FSolver::Normalize(Grid, Chart);
		}
		
		Timer.Mark(Result.Stats.NormalizeMs);
	}

	// Split mesh
	VEUV::FEigenMesh  SeamSplitMesh;
	TArray<FVector2f> VertexUVs;

	// Now create the actual mesh, split it by the effective voxel seams
	// Interpolation between seams is not well-formed
	if (Config.UVVisualizeMode == EVEUVVisualizationDebugMode::None)
	{
		SplitVertexVoxelIndices.Reset();
		SplitVertexSources.Reset();
		
		VEUV::FSplitMesh::SplitByChartSeams(Grid, SolveContext, Mesh, SeamSplitMesh, SplitVertexVoxelIndices, SplitVertexSources);
		Timer.Mark(Result.Stats.SplitFacesChartMs);
	}
	else
	{
		// For debug modes, just use the split mesh, easier to reason for
		SeamSplitMesh = SplitMesh;
	}

	// TODO: Do we *really* need this?
	Result.FaceChartIndices.SetNumUninitialized(SeamSplitMesh.Faces.Num());
	for (int32 FaceIdx = 0; FaceIdx < SeamSplitMesh.Faces.Num(); FaceIdx++)
	{
		const Eigen::Vector3i& Face = SeamSplitMesh.Faces[FaceIdx];
		int32 VoxelIdx = SplitVertexVoxelIndices.IsValidIndex(Face.x()) ? SplitVertexVoxelIndices[Face.x()] : INDEX_NONE;
		Result.FaceChartIndices[FaceIdx] = (VoxelIdx != INDEX_NONE && SolveContext.VoxelChartIndices.IsValidIndex(VoxelIdx))
			? SolveContext.VoxelChartIndices[VoxelIdx]
			: INDEX_NONE;
	}

	// Project the trilinear UVs down to the mesh UVs
	// Please note that this isn't an exact mapping, barycentric interpolation of trilinearly evaluated vertex UVs will not yield
	// the same UVs as those evaluated on each sample. This is expected.
	// TODO: What we need going forward is an additional threshold on which to cut, if the projection error is too great, sub-divide.
	AssignVertexUVs(Config, Grid, SolveContext, SplitMesh, SeamSplitMesh, SplitVertexVoxelIndices, VertexUVs);
	Timer.Mark(Result.Stats.EvalUVsMs);

	// Complete the stats
	Timer.Total(Result.Stats.TotalMs);
	CreateStatistics(Result.Stats, Config, Grid, Mesh, SeamSplitMesh, SolveContext, Norm);

	// Restore the original mesh space
	Norm.Denormalize(SeamSplitMesh.Vertices);
	Result.OutputMesh.Vertices = FromEigen(SeamSplitMesh.Vertices);
	Result.OutputMesh.Faces    = FromEigen(SeamSplitMesh.Faces);
	Result.VertexUVs           = MoveTemp(VertexUVs);
	Result.VertexSources       = MoveTemp(SplitVertexSources);

	// Create charts
	Result.Charts.SetNum(SolveContext.Charts.Num());
	for (int32 I = 0; I < SolveContext.Charts.Num(); I++)
	{
		const VEUV::FChartContext& Src = SolveContext.Charts[I];
		VEUV::FChart& Dst = Result.Charts[I];
		Dst.VoxelIndices = Src.VoxelIndices;
		Dst.Bounds       = FBox2f(FVector2f(Src.AABounds.Min.x(), Src.AABounds.Min.y()), FVector2f(Src.AABounds.Max.x(), Src.AABounds.Max.y()));
		Dst.PackedBounds = FBox2f(FVector2f(Src.PackedAABounds.Min.x(), Src.PackedAABounds.Min.y()), FVector2f(Src.PackedAABounds.Max.x(), Src.PackedAABounds.Max.y()));
		Dst.Area         = Src.Area;
	}

	// Detect failure / inversion and aggregate per-chart status into Result.Status
	ValidateResult(Result, SolveContext);

	// If requested, create debug capture for visualization
	if (Capture)
	{
		FinalizeDebugCapture(Config, Result, Grid, SolveContext, Norm, Capture);
	}

	return MoveTemp(Result);
}

VEUV::FResult VEUV::FOptimizer::Compute(const FMesh& InputMesh, const FVEUVConfig& Config)
{
	const int32 ProfileIterations = FMath::Max(1, CVarVEUVProfileIterations.GetValueOnAnyThread());

	FResult Result;
	for (int32 i = 0; i < ProfileIterations; i++)
	{
		Result = GenerateInternal(InputMesh, Config);
	}
	
	return Result;
}
