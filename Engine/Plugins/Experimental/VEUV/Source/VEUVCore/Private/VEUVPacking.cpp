// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVPacking.h"
#include "TextureLayout.h"
#include "Async/ParallelFor.h"

static VEUV::FEVBounds ComputeEVBounds(const Eigen::MatrixXf& SampleData)
{
	// Create centered samples
	Eigen::RowVectorXf Mean = SampleData.colwise().mean();
	Eigen::MatrixXf Centered = SampleData.rowwise() - Mean;

	// Compute EVs
	Eigen::MatrixXf Cov = Centered.transpose() * Centered * (1.0f / (SampleData.rows() - 1));
	Eigen::SelfAdjointEigenSolver<Eigen::MatrixXf> Solver(Cov);
	Eigen::MatrixXf EV = Solver.eigenvectors().real();

	// For now, just select the two primary EVs
	VEUV::FEVBounds Out;
	Out.Mean = Mean;
	Out.EV0 = EV.col(EV.cols() - 1).normalized();
	Out.EV1 = EV.col(EV.cols() - 2).normalized();

	// Compute projections on two primary EVs
	float D0 = 0, D1 = 0;
	for (int32 I = 0; I < SampleData.rows(); I++)
	{
		D0 = std::max(D0, std::abs(Out.EV0.dot(SampleData.row(I).transpose() - Out.Mean)));
		D1 = std::max(D1, std::abs(Out.EV1.dot(SampleData.row(I).transpose() - Out.Mean)));
	}

	// Scale EVs by the centered samples
	Out.EV0 *= D0;
	Out.EV1 *= D1;
	
	return Out;
}

static void SampleChartUVs(
	VEUV::FVoxelGrid& Grid, 
	VEUV::FChartContext& Chart, 
	TArray<Eigen::Vector2f>& UVs)
{
	TArray<uint32, TInlineAllocator<64>> SamplePrefixes;
	
	// Get sample prefix array
	uint32 TotalSampleCount = 0;
	for (int32 VoxelIndex : Chart.VoxelIndices)
	{
		const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		SamplePrefixes.Add(TotalSampleCount);
		TotalSampleCount += Voxel.Samples.Num();
	}
	
	UVs.SetNumUninitialized(TotalSampleCount);
	
	// Visit voxels within chart
	ParallelFor(Chart.VoxelIndices.Num(), [&](int32 VoxelChartIndex)
	{
		int32 VoxelIndex = Chart.VoxelIndices[VoxelChartIndex];
			
		// Get the prefix
		const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		uint32 SamplePrefix = SamplePrefixes[VoxelChartIndex];

		// Sample UVs for all samples
		ParallelFor(Voxel.Samples.Num(), [&](int32 SampleIndex)
		{
			const VEUV::FSampleData& Sample = Voxel.Samples[SampleIndex];
			
			uint32 WriteOffset = SamplePrefix + SampleIndex;
			UVs[WriteOffset] = Grid.SampleUV(Chart.SolvedUV, VoxelIndex, Sample.Position);
		});
	});
}

static void SampleChartAABounds(
	VEUV::FVoxelGrid& Grid,
	VEUV::FChartContext& Chart,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces)
{
	TArray<VEUV::FAABounds> VoxelAA;
	VoxelAA.SetNum(Chart.VoxelIndices.Num());
	
	// Visit voxels within chart
	ParallelFor(Chart.VoxelIndices.Num(), [&](int32 VoxelChartIndex)
	{
		int32 VoxelIndex = Chart.VoxelIndices[VoxelChartIndex];
		const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		
		// Collect unique vertices
		TSet<uint32> UniqueVertices;
		for (int32 FaceIdx : Voxel.Faces)
		{
			const Eigen::Vector3i& Face = Faces[FaceIdx];
			UniqueVertices.Add(Face.x());
			UniqueVertices.Add(Face.y());
			UniqueVertices.Add(Face.z());
		}
		
		TArray<uint32> Iterable = UniqueVertices.Array();
			
		TArray<Eigen::Vector2f> UVs;
		UVs.SetNumUninitialized(Iterable.Num());
	
		// Build UVs in parallel
		ParallelFor(Iterable.Num(), [&](int32 VertexIndex)
		{
			UVs[VertexIndex] = Grid.SampleUV(Chart.SolvedUV, VoxelIndex, Vertices[Iterable[VertexIndex]]);
		});
			
		VEUV::FAABounds& Bounds = VoxelAA[VoxelChartIndex];

		// Get the bounds
		for (const Eigen::Vector2f& UV : UVs)
		{
			Bounds.Min = Bounds.Min.cwiseMin(UV);
			Bounds.Max = Bounds.Max.cwiseMax(UV);
		}
	});
	
	// Reduce voxel bounds
	for (int32 i = 0; i < Chart.VoxelIndices.Num(); i++)
	{
		Chart.AABounds.Min = Chart.AABounds.Min.cwiseMin(VoxelAA[i].Min);
		Chart.AABounds.Max = Chart.AABounds.Max.cwiseMax(VoxelAA[i].Max);
	}
}

void VEUV::FPacking::ReorientCharts(
	FVoxelGrid& Grid,
	FGlobalSolveContext& SolveContext,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces)
{
	// Pack charts in parallel
	ParallelFor(SolveContext.Charts.Num(), [&](int32 ChartIndex)
	{
		FChartContext& Chart = SolveContext.Charts[ChartIndex];
		
		// Sample UVs
		TArray<Eigen::Vector2f> UVs;
		SampleChartUVs(Grid, Chart, UVs);

		// Traverse voxels in chart
		for (int32 VoxelIndex : Chart.VoxelIndices)
		{
			const FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
			
			// Accumulate triangle area
			for (int32 FaceIdx : Voxel.Faces)
			{
				const Eigen::Vector3i& Face = Faces[FaceIdx];
				Chart.Area += 0.5f * (Vertices[Face.y()] - Vertices[Face.x()]).cross(Vertices[Face.z()] - Vertices[Face.x()]).norm();
			}
		}

		// Create sample set
		Eigen::MatrixXf Samples(UVs.Num(), 2);
		for (int32 I = 0; I < UVs.Num(); I++)
		{
			Samples.row(I) = UVs[I];
		}

		// Extract the EV bounds
		Chart.EVBounds = ComputeEVBounds(Samples);

		// Rotate the bounds so they're axis aligned
		Eigen::Rotation2Df Rot(-std::atan2(Chart.EVBounds.EV0.y(), Chart.EVBounds.EV0.x()));
		Eigen::Vector2f MeanVec(Chart.EVBounds.Mean.x(), Chart.EVBounds.Mean.y());

		// Apply the transformation on all UVs
		for (int32 i = 0; i < Chart.SolvedUV.size() / 2; i++)
		{
			Eigen::Vector2f UV(Chart.SolvedUV[i * 2 + 0], Chart.SolvedUV[i * 2 + 1]);
			UV = MeanVec + Rot * (UV - MeanVec);
			Chart.SolvedUV[i * 2 + 0] = UV.x();
			Chart.SolvedUV[i * 2 + 1] = UV.y();
		}
		
		// Transform EVs
		Chart.EVBounds.Mean = MeanVec + Rot * (Chart.EVBounds.Mean - MeanVec);
		Chart.EVBounds.EV0 = Rot * Chart.EVBounds.EV0;
		Chart.EVBounds.EV1 = Rot * Chart.EVBounds.EV1;

		// Axis aligned bounds
		Chart.AABounds.Min = Eigen::Vector2f::Constant(FLT_MAX);
		Chart.AABounds.Max = Eigen::Vector2f::Constant(-FLT_MAX);

		// Re-sample the rotated UVs for their bounds
		SampleChartAABounds(Grid, Chart, Vertices, Faces);
	});
	
	// Aggregate total area
	float TotalArea = 0.0f;
	for (FChartContext& Chart : SolveContext.Charts)
	{
		TotalArea += Chart.Area;
	}

	// Normalize chart areas
	if (TotalArea > 0.0f)
	{
		for (FChartContext& Chart : SolveContext.Charts)
		{
			Chart.Area /= TotalArea;
		}
	}
}

void VEUV::FPacking::PackCharts(
	FVoxelGrid& Grid,
	FGlobalSolveContext& SolveContext)
{
	uint32 AtlasWidth = static_cast<uint32>(Grid.Config.Packing.AtlasPackWidth);

	// TODO: Prototype logic, not very clean
	
	SolveContext.bPackingSucceeded = false;

	// Power iteration over the domain
	for (float Power = 2.0f; Power > 0.1f; Power -= 0.1f)
	{
		bool bAnyFailed = false;
		
		// Standard packer
		FTextureLayout Layout(1, 1, AtlasWidth, AtlasWidth, false);
		TArray<Eigen::VectorXf> AtlasUVs;

		// Pack all charts into the layout
		for (int32 ChartIdx = 0; ChartIdx < SolveContext.Charts.Num(); ChartIdx++)
		{
			FChartContext& Chart = SolveContext.Charts[ChartIdx];

			// Get the normalized extent, the only thing that matters is that it preserves the relative area in UV space
			// and its w/h extent ratio
			Eigen::Vector2f ProportionalExtent = (Chart.AABounds.Max - Chart.AABounds.Min).normalized();
			Eigen::Vector2f NormExtent = std::sqrt(Chart.Area) * ProportionalExtent * Power;

			// To atlas space
			uint32 Width = FMath::Max(1u, static_cast<uint32>(NormExtent.x() * AtlasWidth));
			uint32 Height = FMath::Max(1u, static_cast<uint32>(NormExtent.y() * AtlasWidth));

			// Try to allocate it in the packer
			uint32 PosX = 0, PosY = 0;
			if (!Layout.AddElement(PosX, PosY, Width, Height))
			{
				bAnyFailed = true;
				break;
			}

			// Back to normalized
			Eigen::Vector2f PackedOffset(static_cast<float>(PosX) / AtlasWidth, static_cast<float>(PosY) / AtlasWidth);
			Eigen::Vector2f PackedScale(static_cast<float>(Width) / AtlasWidth, static_cast<float>(Height) / AtlasWidth);
			Eigen::VectorXf AtlasUV(Chart.SolvedUV.size());
			
			Eigen::Vector2f AAExtent = Chart.AABounds.Max - Chart.AABounds.Min;

			// Transform the UVs to its allocated space
			for (int32 i = 0; i < Chart.SolvedUV.size() / 2; i++)
			{
				Eigen::Vector2f UV(Chart.SolvedUV[i * 2 + 0], Chart.SolvedUV[i * 2 + 1]);
				UV = PackedOffset + (UV - Chart.AABounds.Min).cwiseQuotient(AAExtent).cwiseProduct(PackedScale);
				AtlasUV[i * 2 + 0] = UV.x();
				AtlasUV[i * 2 + 1] = UV.y();
			}
			
			// Transform EVs
			Chart.PackedEVBounds.Mean = PackedOffset + (Chart.EVBounds.Mean - Chart.AABounds.Min).cwiseQuotient(AAExtent).cwiseProduct(PackedScale);
			Chart.PackedEVBounds.EV0 = Chart.EVBounds.EV0.cwiseQuotient(AAExtent).cwiseProduct(PackedScale);
			Chart.PackedEVBounds.EV1 = Chart.EVBounds.EV1.cwiseQuotient(AAExtent).cwiseProduct(PackedScale);

			// Add them
			Chart.PackedAABounds.Min = PackedOffset;
			Chart.PackedAABounds.Max = PackedOffset + PackedScale;
			AtlasUVs.Add(AtlasUV);
		}

		// If any packing failed, just abandon this iteration
		if (bAnyFailed)
		{
			continue;
		}

		// Overwrite the UVs
		for (int32 i = 0; i < SolveContext.Charts.Num(); i++)
		{
			SolveContext.Charts[i].SolvedUV = AtlasUVs[i];
		}

		SolveContext.bPackingSucceeded = true;
		break;
	}
}
