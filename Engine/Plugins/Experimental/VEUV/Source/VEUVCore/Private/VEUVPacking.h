// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVVoxelGrid.h"
#include "VEUVCharting.h"

namespace VEUV
{
	class FPacking
	{
	public:
		/**
		 * Reorient the charts to be axis aligned, rotation determined by primary EVs
		 * @param Grid Parent grid
		 * @param SolveContext Solution to orient against
		 * @param Vertices All vertex
		 * @param Faces All faces
		 */
		static void ReorientCharts(
			FVoxelGrid& Grid,
			FGlobalSolveContext& SolveContext,
			const TArray<Eigen::Vector3f>& Vertices,
			const TArray<Eigen::Vector3i>& Faces
		);

		/**
		 * Pack a chart atlas
		 * @param Grid Parent grid
		 * @param SolveContext Solution to pack
		 */
		static void PackCharts(
			FVoxelGrid& Grid,
			FGlobalSolveContext& SolveContext
		);
	};
}
