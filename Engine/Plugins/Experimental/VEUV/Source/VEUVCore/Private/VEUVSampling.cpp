// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVSampling.h"
#include "VEUVGeometry.h"
#include "Async/ParallelFor.h"
#include <random>

/**
 * This file in particular needs refactoring
 */

struct FComplexityField
{
	TArray<float> Values;
	Eigen::Vector3f Min = Eigen::Vector3f::Zero();
	Eigen::Vector3f Max = Eigen::Vector3f::Zero();
	Eigen::Vector3i Resolution = Eigen::Vector3i::Zero();

	float Sample(const Eigen::Vector3f& Position) const
	{
		if ((Resolution.array() <= 0).any())
		{
			return 0.0f;
		}

		// Normalize the position
		Eigen::Vector3f Res = Resolution.cast<float>();
		Eigen::Vector3f NormPos = (Position - Min).cwiseQuotient(Max - Min).cwiseProduct(Res);
		
		// Get base cell
		Eigen::Vector3f CenterPos = (NormPos - Eigen::Vector3f::Constant(0.5f)).cwiseMax(Eigen::Vector3f::Zero()).cwiseMin(Res - Eigen::Vector3f::Ones());
		Eigen::Vector3i Base = CenterPos.cast<int>().cwiseMin(Resolution - Eigen::Vector3i::Constant(2));
		
		// Position within the cell
		Eigen::Vector3f CellPos = CenterPos - Base.cast<float>();

		float W[8];
		VEUV::Geometry::GetTrilinearWeights(CellPos.x(), CellPos.y(), CellPos.z(), W);

		float Complexity = 0.0f;
		
		// Unrolled
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					Complexity += W[i * 4 + j * 2 + k] * Values[CellIndex(Base.x() + i, Base.y() + j, Base.z() + k)];
				}
			}
		}
		
		return Complexity;
	}
	
	int32 CellIndex(int32 X, int32 Y, int32 Z) const
	{
		return Z * Resolution.y() * Resolution.x() + Y * Resolution.x() + X; 
	}

	void Normalize()
	{
		float MaxValue = 0.0f;
		for (float Value : Values)
		{
			MaxValue = FMath::Max(MaxValue, Value);
		}

		if (MaxValue < 1e-10f)
		{
			return;
		}

		float Inv = 1.0f / MaxValue;
		for (float& Value : Values)
		{
			Value *= Inv;
		}
	}
};

struct FComplexityDirectionBin
{
	TArray<int32> Faces;
	TArray<float> Weights;
};

static FComplexityField BuildComplexityField(
	const VEUV::FVoxelGrid& Grid,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces,
	const FVEUVConfig& Config,
	const VEUV::FNormalization& Norm)
{
	FComplexityField Field;
	Field.Min = Grid.LogicalMin;
	Field.Max = Grid.LogicalMax;

	Eigen::Vector3f Extent = Field.Max - Field.Min;
	float CellSize = FMath::Max(Config.Sampling.ComplexityFieldCellSize, 1e-6f) / FMath::Max(Norm.Scale, 1e-6f);
	int32 MaxRes = FMath::Max(2, Config.Sampling.ComplexityFieldMaxResolution);

	// Set resolution
	Field.Resolution = Eigen::Vector3i(
		FMath::Clamp(FMath::CeilToInt32(Extent.x() / CellSize), 2, MaxRes),
		FMath::Clamp(FMath::CeilToInt32(Extent.y() / CellSize), 2, MaxRes),
		FMath::Clamp(FMath::CeilToInt32(Extent.z() / CellSize), 2, MaxRes));

	// Allocate cells
	int32 NumCells = Field.Resolution.x() * Field.Resolution.y() * Field.Resolution.z();
	Field.Values.SetNumZeroed(NumCells);

	// Working sums
	TArray<Eigen::Vector3f> CellNormalSum;
	TArray<float> CellTotalArea;
	CellNormalSum.SetNumZeroed(NumCells);
	CellTotalArea.SetNumZeroed(NumCells);

	// Accumulate per face
	for (const Eigen::Vector3i& Face : Faces)
	{
		const Eigen::Vector3f& P0 = Vertices[Face.x()];
		const Eigen::Vector3f& P1 = Vertices[Face.y()];
		const Eigen::Vector3f& P2 = Vertices[Face.z()];

		// E0xE2
		Eigen::Vector3f Cross = (P1 - P0).cross(P2 - P0);
		
		// Ignore degenerate triangles
		float Area = 0.5f * Cross.norm();
		if (Area < FLT_EPSILON)
		{
			continue;
		}
		
		// Get the centroid
		Eigen::Vector3f Centroid     = (P0 + P1 + P2) / 3.0f;
		Eigen::Vector3f NormCentroid = (Centroid - Field.Min).cwiseQuotient(Extent);

		// To cell index
		Eigen::Vector3i Cell = 
			NormCentroid.cwiseProduct(Field.Resolution.cast<float>()).cast<int>()
			.cwiseMax(Eigen::Vector3i::Zero()).cwiseMin(Field.Resolution - Eigen::Vector3i::Ones());

		// Accumulate
		int32 CellIdx = Field.CellIndex(Cell.x(), Cell.y(), Cell.z());
		CellNormalSum[CellIdx] += Area * Cross.normalized();
		CellTotalArea[CellIdx] += Area;
	}
	
	// Compute field values
	float Alpha = FMath::Max(0.0f, Config.Sampling.ComplexityAlpha);
	for (int32 I = 0; I < NumCells; I++)
	{
		if (CellTotalArea[I] > FLT_EPSILON)
		{
			float Dispersion = 1.0f - CellNormalSum[I].norm() / CellTotalArea[I];
			Field.Values[I] = Alpha * Dispersion;
		}
	}

	// Naive dilution passes
	for (int32 Iter = 0; Iter < Config.Sampling.ComplexityDilutionIterations; Iter++)
	{
		constexpr float Decay = 0.5f;
		
		static const Eigen::Vector3i DilutionOffsets[] = {
			{-1,0,0},
			{1,0,0},
			{0,-1,0},
			{0,1,0},
			{0,0,-1},
			{0,0,1}
		};
		
		TArray<float> WorkingValues;
		WorkingValues.SetNumZeroed(NumCells);

		// Dilute each pass by their neighbours
		for (int32 z = 0; z < Field.Resolution.z(); z++)
		{
			for (int32 y = 0; y < Field.Resolution.y(); y++)
			{
				for (int32 x = 0; x < Field.Resolution.x(); x++)
				{
					float MaxNeighbouring = 0.0f;
					
					// Accumulate on neighbours
					for (const Eigen::Vector3i& Offset : DilutionOffsets)
					{
						Eigen::Vector3i Pos = Eigen::Vector3i(x, y, z) + Offset;
						
						if ((Pos.array() >= 0).all() && (Pos.array() < Field.Resolution.array()).all())
						{
							MaxNeighbouring = FMath::Max(MaxNeighbouring, Field.Values[Field.CellIndex(Pos.x(), Pos.y(), Pos.z())]);
						}
					}

					// Max it
					int32 Idx = Field.CellIndex(x, y, z);
					WorkingValues[Idx] = FMath::Max(Field.Values[Idx], MaxNeighbouring * Decay);
				}
			}
		}

		Field.Values = MoveTemp(WorkingValues);
	}

	// Normalize the field, otherwise we cant mix different weighting
	Field.Normalize();
	
	// Final ramping
	for (float& Value : Field.Values)
	{
		Value = FMath::Pow(Value, 2.0f);
	}

	return Field;
}

static void ComputeFaceWeights(
	VEUV::FVoxelGrid& Grid,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces,
	const FComplexityField& Field)
{
	// Max area for normalization
	float MaxArea = 0.0f;
	for (int32 FaceIdx = 0; FaceIdx < Faces.Num(); FaceIdx++)
	{
		const Eigen::Vector3i& Face = Faces[FaceIdx];
		const Eigen::Vector3f& P0 = Vertices[Face.x()];
		const Eigen::Vector3f& P1 = Vertices[Face.y()];
		const Eigen::Vector3f& P2 = Vertices[Face.z()];
		MaxArea = FMath::Max(MaxArea, (P1 - P0).cross(P2 - P0).norm() * 0.5f);
	}

	// May be degenerate
	if (MaxArea < FLT_EPSILON)
	{
		MaxArea = 1.0f;
	}

	// Visit voxels in parallel
	ParallelFor(Grid.Voxels.Num(), [&](int32 VoxelIndex)
	{
		VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		Voxel.FaceAreaWeights.SetNumUninitialized(Voxel.Faces.Num());
		Voxel.FaceComplexityWeights.SetNumUninitialized(Voxel.Faces.Num());
			
		// Compute weights for all faces
		ParallelFor(Voxel.Faces.Num(), [&](int32 FaceIndex)
		{
			const Eigen::Vector3i& Face = Faces[Voxel.Faces[FaceIndex]];
			const Eigen::Vector3f& P0 = Vertices[Face.x()];
			const Eigen::Vector3f& P1 = Vertices[Face.y()];
			const Eigen::Vector3f& P2 = Vertices[Face.z()];
		
			// Add area weight
			float NormArea = ((P1 - P0).cross(P2 - P0).norm() * 0.5f) / MaxArea;
			Voxel.FaceAreaWeights[FaceIndex] = NormArea;
	
			// Add normal complexity weight
			float Complexity = Field.Sample((P0 + P1 + P2) / 3.0f);
			Voxel.FaceComplexityWeights[FaceIndex] = Complexity;
		});
	});
}

static void CreateFibonacciGABins(int32 Count, TArray<Eigen::Vector3f>& OutDirections)
{
	OutDirections.SetNum(Count);

	// Standard golden angle fibonacci
	float GoldenAngle = PI * (3.0f - std::sqrt(5.0f));
	
	for (int32 i = 0; i < Count; i++)
	{
		float Z = 1.0f - (2.0f * i + 1.0f) / static_cast<float>(Count);
		float R = std::sqrt(1.0f - Z * Z);
		float Theta = GoldenAngle * static_cast<float>(i);
		OutDirections[i] = Eigen::Vector3f(R * std::cos(Theta), R * std::sin(Theta), Z);
	}
}

static int32 FindNearestBin(const Eigen::Vector3f& Normal, const TArray<Eigen::Vector3f>& Directions)
{
	int32 Candidate  = 0;
	float CandidateW = -FLT_MAX;

	// Just do a linear search over the directions
	for (int32 i = 0; i < Directions.Num(); i++)
	{
		float W = std::abs(Normal.dot(Directions[i]));
		if (W > CandidateW)
		{
			CandidateW = W;
			Candidate  = i;
		}
	}

	return Candidate;
}

static bool SampleFace(
	std::mt19937& Generator,
	std::uniform_real_distribution<float>& Dist,
	const TArray<Eigen::Vector3f>& Vertices,
	VEUV::FVoxelData& Voxel,
	const Eigen::Vector3i& Face,
	const Eigen::Vector3f& Min,
	const Eigen::Vector3f& Max,
	EVEUVSampleType Type)
{
	float U = Dist(Generator);
	float V = Dist(Generator);

	float B0 = 1.0f - std::sqrt(U);
	float B1 = std::sqrt(U) * (1.0f - V);
	float B2 = std::sqrt(U) * V;

	const Eigen::Vector3f& P0 = Vertices[Face.x()];
	const Eigen::Vector3f& P1 = Vertices[Face.y()];
	const Eigen::Vector3f& P2 = Vertices[Face.z()];

	Eigen::Vector3f P = B0 * P0 + B1 * P1 + B2 * P2;

	if ((P.array() < Min.array()).any() || (P.array() > Max.array()).any())
	{
		return false;
	}

	VEUV::FSampleData& Sample = Voxel.Samples.AddDefaulted_GetRef();
	Sample.Position = P;
	Sample.Normal = (P1 - P0).cross(P2 - P0).normalized();
	Sample.Type = Type;
	
	Voxel.SampleAverageNormal += Sample.Normal;
	return true;
}

static void SampleSingleVoxel(
	VEUV::FVoxelData& Voxel,
	const FVEUVConfig& Config,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces,
	int32 NumSamples)
{
	constexpr int32 MaxRejectionMultiplier = 64;
	
	// Reset state
	Voxel.SampleAverageNormal = Eigen::Vector3f::Zero();

	// No work?
	if (Voxel.Faces.IsEmpty() || NumSamples <= 0)
	{
		return;
	}

	// Initial seed for determinism
	// TODO: With norm this is wrong
	uint32 Seed = static_cast<uint32>(
		std::hash<float>()(Voxel.LogicalMin.x()) ^
		std::hash<float>()(Voxel.LogicalMin.y()) ^
		std::hash<float>()(Voxel.LogicalMin.z())
	);
	
	// Shared generator
	std::mt19937 Generator(Seed);
	
	// Use a uniform distribution for barycentrics
	std::uniform_real_distribution<float> BarycentricDistribution(0.0f, 1.0f);

	// We split the sampling pool based on the blend
	int32 ComplexitySamples = FMath::RoundToInt32(NumSamples * Config.Sampling.ComplexityBlend);
	int32 AreaSamples       = NumSamples - ComplexitySamples;
	
	// Area sampling
	{
		// Area CDF
		std::discrete_distribution<int> AreaDist(Voxel.FaceAreaWeights.GetData(), Voxel.FaceAreaWeights.GetData() + Voxel.FaceAreaWeights.Num());

		// Area pool
		int32 MaxAttempts = AreaSamples * MaxRejectionMultiplier;
		int32 Count = 0;
		int32 Attempts = 0;
		
		while (Count < AreaSamples && Attempts < MaxAttempts)
		{
			Attempts++;
			
			uint32 FaceIndex = Voxel.Faces[AreaDist(Generator)];
			const Eigen::Vector3i& Face = Faces[FaceIndex];

			if (SampleFace(Generator, BarycentricDistribution, Vertices, Voxel, Face, Voxel.LogicalMin, Voxel.LogicalMax, EVEUVSampleType::Area))
			{
				Count++;
			}
		}
	}

	// Complexity sampling
	if (ComplexitySamples > 0)
	{
		TArray<FComplexityDirectionBin> Bins;
		Bins.SetNum(Config.Sampling.ComplexityNormalBins);
		
		// Generate bin directions
		TArray<Eigen::Vector3f> BinDirections;
		CreateFibonacciGABins(Config.Sampling.ComplexityNormalBins, BinDirections);
		
		// Accumulate
		for (int32 i = 0; i < Voxel.Faces.Num(); i++)
		{
			const Eigen::Vector3i& Face = Faces[Voxel.Faces[i]];
			const Eigen::Vector3f& P0 = Vertices[Face.x()];
			const Eigen::Vector3f& P1 = Vertices[Face.y()];
			const Eigen::Vector3f& P2 = Vertices[Face.z()];
			Eigen::Vector3f N = (P1 - P0).cross(P2 - P0);
			
			// Ignore if degenerate
			if (N.norm() < FLT_EPSILON)
			{
				continue;
			}
			
			// Append to bin
			int32 BinIdx = FindNearestBin(N.normalized(), BinDirections);
			Bins[BinIdx].Faces.Add(Voxel.Faces[i]);
			Bins[BinIdx].Weights.Add(Voxel.FaceComplexityWeights[i]);
		}

		// Number of useful bins
		int32 ActiveBins = 0;
		for (const FComplexityDirectionBin& Bin : Bins)
		{
			if (!Bin.Faces.IsEmpty())
			{
				ActiveBins++;
			}
		}

		// TODO: We need to weigh the budget by the bin face count no?
		if (ActiveBins > 0)
		{
			// Distribute the budget across active bins; spread the floor remainder
			// over the first bins so we use the full budget even when ComplexitySamples < ActiveBins
			int32 MinBudgetPerBin = ComplexitySamples / ActiveBins;
			int32 ExtraSamples = ComplexitySamples - MinBudgetPerBin * ActiveBins;

			for (FComplexityDirectionBin& Bin : Bins)
			{
				if (Bin.Faces.IsEmpty())
				{
					continue;
				}

				int32 BinBudget = MinBudgetPerBin + int32(ExtraSamples-- > 0);

				// Weight CDF
				std::discrete_distribution<int> BinDist(Bin.Weights.GetData(), Bin.Weights.GetData() + Bin.Weights.Num());

				// Complexity pool
				int32 MaxAttempts = BinBudget * MaxRejectionMultiplier;
				int32 Count = 0;
				int32 Attempts = 0;

				while (Count < BinBudget && Attempts < MaxAttempts)
				{
					Attempts++;
					
					uint32 FaceIndex = Bin.Faces[BinDist(Generator)];
					const Eigen::Vector3i& Face = Faces[FaceIndex];

					if (SampleFace(Generator, BarycentricDistribution, Vertices, Voxel, Face, Voxel.LogicalMin, Voxel.LogicalMax, EVEUVSampleType::Complexity))
					{
						Count++;
					}
				}
			}
		}
	}

	// Normalize if any samples
	if (!Voxel.Samples.IsEmpty())
	{
		Voxel.SampleAverageNormal.normalize();
	}
}

float VEUV::FSampling::EvaluateSampleErrorR78(
	const FVoxelGrid& Grid,
	const FSolveRegion& Region,
	const FVoxelData& Voxel,
	const FSampleData& Sample)
{
	Eigen::Vector3f VoxelExtent = Voxel.LogicalMax - Voxel.LogicalMin;
	Eigen::Vector3f NormPos     = (Sample.Position - Voxel.LogicalMin).cwiseQuotient(VoxelExtent);

	Eigen::Vector3f UVPD[8];
	Geometry::GetTrilinearPartialXYZDerivatives(NormPos.x(), NormPos.y(), NormPos.z(), UVPD);

	Eigen::Vector3f GradU = Eigen::Vector3f::Zero();
	Eigen::Vector3f GradV = Eigen::Vector3f::Zero();

	// Accumulate gradients
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				if (CornerIdx == INDEX_NONE)
				{
					return 0.0f;
				}

				Eigen::Vector3f J_c = UVPD[i * 4 + j * 2 + k].cwiseQuotient(VoxelExtent);
				GradU += Region.SolvedUV[CornerIdx * 2 + 0] * J_c;
				GradV += Region.SolvedUV[CornerIdx * 2 + 1] * J_c;
			}
		}
	}

	// Evaluate energy terms
	Eigen::Vector3f R7 = Sample.Normal.cross(GradU) - GradV;
	Eigen::Vector3f R8 = -GradU + GradV.cross(Sample.Normal);

	// Treat them as absolute errors
	return R7.squaredNorm() + R8.squaredNorm();
}

static int32 AdaptiveSampleSubCell(
	VEUV::FVoxelData& Voxel,
	const Eigen::Vector3f& SubCellMin,
	const Eigen::Vector3f& SubCellMax,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces,
	int32 NumSamples)
{
	constexpr int32 MaxRejectionMultiplier = 32;
	
	// No work?
	if (Voxel.Faces.IsEmpty() || NumSamples <= 0)
	{
		return 0;
	}
	
	// Initial seed for determinism
	// TODO: With norm this is wrong
	uint32 Seed = static_cast<uint32>(
		std::hash<float>()(Voxel.LogicalMin.x()) ^
		std::hash<float>()(Voxel.LogicalMin.y()) ^
		std::hash<float>()(Voxel.LogicalMin.z())
	);

	// Shared generator
	std::mt19937 Gen(Seed);
	
	// Use a uniform distribution for barycentrics
	std::uniform_real_distribution<float> BarycentricDistribution(0.0f, 1.0f);
	
	// Face CDF
	std::discrete_distribution<int> FaceDist(Voxel.FaceAreaWeights.GetData(), Voxel.FaceAreaWeights.GetData() + Voxel.FaceAreaWeights.Num());

	// Face pool
	int32 MaxAttempts = NumSamples * MaxRejectionMultiplier;
	int32 Count = 0;
	int32 Attempts = 0;

	// Keep sampling
	while (Count < NumSamples && Attempts < MaxAttempts)
	{
		Attempts++;
		
		uint32 FaceIndex = Voxel.Faces[FaceDist(Gen)];
		const Eigen::Vector3i& Face = Faces[FaceIndex];

		if (SampleFace(Gen, BarycentricDistribution, Vertices, Voxel, Face, SubCellMin, SubCellMax, EVEUVSampleType::Adaptive))
		{
			Count++;
		}
	}

	return Count;
}

int32 VEUV::FSampling::AdaptiveResample(
	FVoxelGrid& Grid,
	const FVEUVConfig& Config,
	const FNormalization& Norm,
	const FSolveRegion& Region,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces)
{
	// TODO: Just generalize the field, makes this a lot cleaner
	int32 SamplesAdded = 0;
	
	// No work?
	if (Config.Sampling.AdaptiveSubdivisions <= 0 || Config.Sampling.AdaptiveFraction <= 0.0f)
	{
		return 0;
	}
	
	// Count number of relevant voxels
	int32 ActiveVoxels = 0;
	for (const FVoxelData& V : Grid.Voxels)
	{
		if (!V.Samples.IsEmpty())
		{
			ActiveVoxels++;
		}
	}
	
	// Any relevant voxel will always have at least one sample
	if (!ActiveVoxels)
	{
		return 0;
	}
	
	// Total sub-cell count
	int32 NumSubCells = Config.Sampling.AdaptiveSubdivisions * Config.Sampling.AdaptiveSubdivisions * Config.Sampling.AdaptiveSubdivisions;
	
	// Adaptive re-sample all voxels of the solve region in parallel
	ParallelFor(Region.VoxelIndices.Num(), [&](int32 RegionVoxelIndex)
	{
		FVoxelData& Voxel = Grid.Voxels[Region.VoxelIndices[RegionVoxelIndex]];
		if (Voxel.Samples.IsEmpty())
		{
			return;
		}
	
		// Number of adaptive samples we're allowed
		int32 AdaptiveSamples = FMath::Max(1, FMath::RoundToInt32(Voxel.Samples.Num() * Config.Sampling.AdaptiveFraction));

		// Extent of a single sub-cell
		Eigen::Vector3f SubCellExtent = (Voxel.LogicalMax - Voxel.LogicalMin) / static_cast<float>(Config.Sampling.AdaptiveSubdivisions);
		
		// Cell working data
		TArray<float> SubCellError;
		TArray<int32> SubCellSampleCount;
		SubCellError.SetNumZeroed(NumSubCells);
		SubCellSampleCount.SetNumZeroed(NumSubCells);

		// Accumulate error per sample
		for (const FSampleData& Sample : Voxel.Samples)
		{
			Eigen::Vector3f LocalPos = (Sample.Position - Voxel.LogicalMin).cwiseQuotient(SubCellExtent);
			Eigen::Vector3i Cell = LocalPos.cast<int>().cwiseMax(Eigen::Vector3i::Zero()).cwiseMin(Eigen::Vector3i::Constant(Config.Sampling.AdaptiveSubdivisions - 1));
			
			// To cell index
			// TODO: Again, have some generic field helper
			int32 CellIdx = Cell.z() * Config.Sampling.AdaptiveSubdivisions * Config.Sampling.AdaptiveSubdivisions + Cell.y() * Config.Sampling.AdaptiveSubdivisions + Cell.x();

			SubCellError[CellIdx] += EvaluateSampleErrorR78(Grid, Region, Voxel, Sample);
			SubCellSampleCount[CellIdx]++;
		}

		// Normalize cell sample errors
		float TotalWeight = 0.0f;
		for (int32 I = 0; I < NumSubCells; I++)
		{
			if (SubCellSampleCount[I] > 0)
			{
				SubCellError[I] /= static_cast<float>(SubCellSampleCount[I]);
			}
			
			TotalWeight += SubCellError[I];
		}

		if (TotalWeight < FLT_EPSILON)
		{
			return;
		}

		// Adaptive re-sample each cell
		for (int32 z = 0; z < Config.Sampling.AdaptiveSubdivisions; z++)
		{
			for (int32 y = 0; y < Config.Sampling.AdaptiveSubdivisions; y++)
			{
				for (int32 x = 0; x < Config.Sampling.AdaptiveSubdivisions; x++)
				{
					// To cell index
					int32 CellIdx = z * Config.Sampling.AdaptiveSubdivisions * Config.Sampling.AdaptiveSubdivisions + y * Config.Sampling.AdaptiveSubdivisions + x;
					
					// Allocate budget against normalized error
					float NormError       = SubCellError[CellIdx] / TotalWeight;
					int32 CellSampleCount = FMath::RoundToInt32(AdaptiveSamples * NormError);
					
					if (CellSampleCount > 0)
					{
						Eigen::Vector3f SubCellMin = Voxel.LogicalMin + Eigen::Vector3f(x, y, z).cwiseProduct(SubCellExtent);
						Eigen::Vector3f SubCellMax = SubCellMin + SubCellExtent;
						
						int32 LocalSamplesAdded = AdaptiveSampleSubCell(
							Voxel,
							SubCellMin, SubCellMax,
							Vertices, Faces,
							CellSampleCount
						);
						
						FPlatformAtomics::InterlockedAdd(&SamplesAdded, LocalSamplesAdded);
					}
				}
			}
		}

		// Re-normalize sample normal
		// TODO: This is incorrect...
		if (!Voxel.Samples.IsEmpty())
		{
			Voxel.SampleAverageNormal.normalize();
		}
	});

	return SamplesAdded;
}

void VEUV::FSampling::SampleVoxels(
	FVoxelGrid& Grid,
	const FVEUVConfig& Config,
	const FNormalization& Norm,
	const TArray<Eigen::Vector3f>& Vertices,
	const TArray<Eigen::Vector3i>& Faces)
{
	// Build the normal based complexity field
	FComplexityField Field = BuildComplexityField(Grid, Vertices, Faces, Config, Norm);

	// Assign weights from area and field
	ComputeFaceWeights(Grid, Vertices, Faces, Field);

	// Per voxel weights
	TArray<float> VoxelWeights;
	VoxelWeights.SetNumZeroed(Grid.Voxels.Num());
	
	// Set up per-voxel weights
	float TotalWeight = 0.0f;
	for (int32 VoxelIndex = 0; VoxelIndex < Grid.Voxels.Num(); VoxelIndex++)
	{
		const FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		
		// Just accumulate the existing weights
		float Weight = 0.0f;
		for (int32 FaceIndex = 0; FaceIndex < Voxel.Faces.Num(); FaceIndex++)
		{
			Weight += Voxel.FaceAreaWeights[FaceIndex] + Voxel.FaceComplexityWeights[FaceIndex];
		}
		
		VoxelWeights[VoxelIndex] = Weight;
		TotalWeight += Weight;
	}

	// Sample in parallel
	ParallelFor(Grid.Voxels.Num(), [&](int32 VoxelIndex)
	{
		FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
		if (!Voxel.Faces.IsEmpty())
		{
			// Budget the samples based on the relative weight
			int32 VoxelSamples = FMath::Max(1, FMath::RoundToInt32(Config.Sampling.TotalSamples * VoxelWeights[VoxelIndex] / TotalWeight));

			// Actually sample the voxel
			SampleSingleVoxel(Voxel, Config, Vertices, Faces, VoxelSamples);
		}
	});
	
	// Allocate corners
	for (FVoxelData& Voxel : Grid.Voxels) 
	{
		// No faces, no corner allocations
		// Greatly simplifies the solving space
		if (Voxel.Faces.IsEmpty())
		{
			continue;
		}

		// Allocate corner indices
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					Grid.AllocateCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				}
			}
		}
	}
}
