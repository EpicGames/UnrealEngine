// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVVoxelGrid.h"
#include "VEUVSolver.h"

namespace VEUV
{
	struct FNormalization;

	class FSampling
	{
	public:
		/**
		 * Sample all voxels
	 	 * @param Grid Parent grid
		 * @param Config Build config
		 * @param Norm Normalization state
		 * @param Vertices Mesh vertex set
		 * @param Faces Mesh face set
		 * @param VertexVoxelIndices All vertex to voxel associations
		 */
		static void SampleVoxels(
			FVoxelGrid& Grid,
			const FVEUVConfig& Config,
			const FNormalization& Norm,
			const TArray<Eigen::Vector3f>& Vertices,
			const TArray<Eigen::Vector3i>& Faces
		);

		/**
		 * Adaptively resample voxels on error
    	 * @param Grid Parent grid
		 * @param Config Build config
		 * @param Norm Normalization state
		 * @param Region The solve region to evaluate error against
		 * @param Vertices Mesh vertex set
		 * @param Faces Mesh face set
		 */
		static int32 AdaptiveResample(
			FVoxelGrid& Grid,
			const FVEUVConfig& Config,
			const FNormalization& Norm,
			const FSolveRegion& Region,
			const TArray<Eigen::Vector3f>& Vertices,
			const TArray<Eigen::Vector3i>& Faces
		);

		/**
		 * Evaluate a sample error
    	 * @param Grid Parent grid
		 * @param Region The solve region to evaluate error against
		 * @param Voxel Voxel of the sample
		 * @param Sample Sample to evaluate
		 * @return 
		 */
		static float EvaluateSampleErrorR78(
			const FVoxelGrid& Grid,
			const FSolveRegion& Region,
			const FVoxelData& Voxel,
			const FSampleData& Sample 
		);
	};
}
