// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVEigen.h"

namespace VEUV::Schedulers
{
	struct FAdam
	{
		float Beta1 = 0.9f;
		float Beta2 = 0.999f;
		
		struct FCoeffInvariantSnapshot
		{
			float MomentAMean = 0.0f;
			float MomentBMean = 0.0f;
			float Time = 0.0f;
		};
		
		void Init(int32 CoeffCount)
		{
			MomentA = Eigen::VectorXf::Zero(CoeffCount);
			MomentB = Eigen::VectorXf::Zero(CoeffCount);
		}

		Eigen::VectorXf Step(Eigen::VectorXf& Gradient, float LR)
		{
			Time += 1.0f;

			// Classical moment with variance (not really) moment
			MomentA = Beta1 * MomentA + (1.0f - Beta1) * Gradient;
			MomentB = Beta2 * MomentB.array() + (1.0f - Beta2) * Gradient.array().square();

			// Bias
			Eigen::VectorXf MAHat = MomentA / (1.0f - std::pow(Beta1, Time));
			Eigen::VectorXf MBHat = MomentB / (1.0f - std::pow(Beta2, Time));

			constexpr float Epsilon = 1e-8f;
			return LR * (MAHat.array() / (MBHat.array().sqrt() + Epsilon)).matrix();
		}
		
		void SetFromSnapshot(const FCoeffInvariantSnapshot& Snapshot)
		{
			MomentA.setConstant(Snapshot.MomentAMean);
			MomentB.setConstant(Snapshot.MomentBMean);
			Time = Snapshot.Time;
		}
		
		FCoeffInvariantSnapshot GetSnapshot()
		{
			FCoeffInvariantSnapshot Out;
			Out.MomentAMean = MomentA.mean();
			Out.MomentBMean = MomentB.mean();
			Out.Time = Time;
			return Out;
		}

		Eigen::VectorXf MomentA;
		Eigen::VectorXf MomentB;
		float Time = 0.0f;
	};

	struct FNormSlopeCondition
	{
		int32 Window = 32;
		float Threshold = 1e-3f;
		float Eps = 1e-12f;

		bool Step(float CurrentMetric)
		{
			History.Add(CurrentMetric);

			// Wait for a full window
			if (History.Num() < Window)
			{
				return false;
			}

			// Prune to window size
			if (History.Num() > Window)
			{
				History.RemoveAt(0);
			}
			
			// Get slope and metric
			float Slope, MeanMetric;
			if (!GetWindowSlopeLS(Slope, MeanMetric))
			{
				return false;
			}
			
			// Stop when the proportional slope is smaller than the given threshold
			float NormSlope = Slope / FMath::Max(FMath::Abs(MeanMetric), Eps);
			return -NormSlope < Threshold;
		}
		
	private:
		bool GetWindowSlopeLS(float& Slope, float& MeanMetric)
		{
			// nE[xy] - E[x]E[y] / nE[x^2] - E[x]^2
			
			float SumX = 0.0f;
			float SumY = 0.0f;
			float SumX2 = 0.0f;
			float SumXY = 0.0f;

			// Just accumulate over the sample window
			for (int32 i = 0; i < Window; ++i)
			{
				float X = static_cast<float>(i);
				float Y = History[i];

				SumX += X;
				SumY += Y;
				SumX2 += X * X;
				SumXY += X * Y;
			}

			// Denominator
			float N = static_cast<float>(Window);
			float D = N * SumX2 - SumX * SumX;

			// Just ignore bad slopes
			if (FMath::Abs(D) < Eps)
			{
				return false;
			}
			
			MeanMetric = SumY / N;
			Slope      = (N * SumXY - SumX * SumY) / D;
			return true;
		}

		TArray<float> History;
	};
}
