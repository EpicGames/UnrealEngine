// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVSolver.h"
#include "VEUVCharting.h"
#include "VEUVGeometry.h"
#include "VEUVSampling.h"
#include "VEUVSchedulers.h"
#include "VEUVValidation.h"
#include "Async/ParallelFor.h"

#if VEUV_VALIDATE
UE_DISABLE_OPTIMIZATION
#endif // VEUV_VALIDATE

struct FResidualMatrix
{
	Eigen::MatrixXf Hessian;
	Eigen::VectorXf G;
};

struct FSampleResidualData
{
	Eigen::Vector3f NormPosition;
	uint32 VoxelIndex;
	Eigen::Vector3f Normal;
	EVEUVSampleType Type;
};

static_assert(sizeof(FSampleResidualData) == 32);

static VEUV::FSparseResidualRowControl& AllocateControl(VEUV::FSparseResidualArray& Array, int32 InfluenceCount, int32 RowCount)
{
	check(InfluenceCount < 255 && RowCount < 255);

	// Validate bounds
	check(Array.ControlOffset < Array.Controls.Num());
	check(Array.IndexOffset + InfluenceCount <= Array.Indices.Num());
	check(Array.RowOffset + RowCount <= Array.Rows.Num());
	check(Array.JacobianOffset + InfluenceCount * RowCount * 2 <= Array.JacobianResiduals.Num());

	// Allocate control
	VEUV::FSparseResidualRowControl& Control = Array.Controls[Array.ControlOffset++];
	Control.InfluenceCount = static_cast<uint8>(InfluenceCount);
	Control.RowCount       = static_cast<uint8>(RowCount);

	// Allocate index span
	Control.InfluenceOffset = Array.IndexOffset;
	Array.IndexOffset += InfluenceCount;

	// Allocate row span
	Control.RowOffset = Array.RowOffset;
	Array.RowOffset += RowCount;

	// Allocate jacobian span
	int32 JacobianOffset = Array.JacobianOffset;
	Array.JacobianOffset += InfluenceCount * RowCount * 2;

	// Set jacobian offsets for each row
	for (int32 i = 0; i < RowCount; i++)
	{
		VEUV::FSparseResidualRow& Row = Array.Rows[Control.RowOffset + i];
		Row.JacobianResidualOffset = JacobianOffset + i * InfluenceCount * 2;
	}

	return Control;
}

static void SortControl(VEUV::FSparseResidualArray& Array, VEUV::FSparseResidualRowControl& Control)
{
	// TODO: Rework the sorting, in most cases we can avoid it by just emitting it in the right order
	for (int32 i = 1; i < Control.InfluenceCount; i++)
	{
		for (int32 j = i; j > 0; j--)
		{
			if (Array.Indices[Control.InfluenceOffset + j - 1] <= Array.Indices[Control.InfluenceOffset + j])
			{
				break;
			}

			Swap(Array.Indices[Control.InfluenceOffset + j - 1], Array.Indices[Control.InfluenceOffset + j]);

			for (int32 RowIndex = 0; RowIndex < Control.RowCount; RowIndex++)
			{
				VEUV::FSparseResidualRow& Row = Array.Rows[Control.RowOffset + RowIndex];

				Swap(Array.JacobianResiduals[Row.JacobianResidualOffset + (j - 1) * 2 + 0],
					 Array.JacobianResiduals[Row.JacobianResidualOffset + j * 2 + 0]);
				Swap(Array.JacobianResiduals[Row.JacobianResidualOffset + (j - 1) * 2 + 1],
					 Array.JacobianResiduals[Row.JacobianResidualOffset + j * 2 + 1]);
			}
		}
	}
}

static void SortArray(VEUV::FSparseResidualArray& Array)
{
	ParallelFor(Array.ControlOffset, [&](int32 i)
	{
		SortControl(Array, Array.Controls[i]);
	});
}

FResidualMatrix CreateResidualMatrix(uint32 UnknownCount)
{
	FResidualMatrix Set;
	Set.Hessian = Eigen::MatrixXf::Zero(UnknownCount * 2, UnknownCount * 2);
	Set.G = Eigen::VectorXf::Zero(UnknownCount * 2);
	return MoveTemp(Set);
}

void MultiplyLocalResidualsWithGlobalScatter(const VEUV::FSparseResidualArray& Array, uint32 ControlOffset, FResidualMatrix& Set)
{
	const int32* RESTRICT IndexData    = Array.Indices.GetData();
	const float* RESTRICT JacobianData = Array.JacobianResiduals.GetData();
	
	float* RESTRICT HessianData = Set.Hessian.data();
	float* RESTRICT GData       = Set.G.data();
	
	const int32 HessianStride = static_cast<int32>(Set.Hessian.outerStride());

	// With the control grouping, we can do local multiplications with global scatters since they always end up in coherent blocks
	for (int32 i = ControlOffset; i < Array.ControlOffset; i++)
	{
		const VEUV::FSparseResidualRowControl& Control = Array.Controls[i];
		const int32 InfluenceCountUV = Control.InfluenceCount * 2;
		
		// May be skipped for parallelism
		if (!InfluenceCountUV)
		{
			continue;
		}

		// Get the global indices
		int32 GlobalIndices[16];
		for (int32 C = 0; C < InfluenceCountUV; C++)
		{
			GlobalIndices[C] = IndexData[Control.InfluenceOffset + C / 2] * 2 + C % 2;
		}

		// Multiply into local states
		// TODO: Maybe Eigen handles the lmul better?
		float LocalHessian[16 * 16] = {};
		float LocalGTarget[16]      = {};

		for (int32 RowIndex = 0; RowIndex < Control.RowCount; RowIndex++)
		{
			const VEUV::FSparseResidualRow& Row = Array.Rows[Control.RowOffset + RowIndex];
			const float* RESTRICT RowJacobian = JacobianData + Row.JacobianResidualOffset;

			// Accumulate targets
			for (int32 InfIndex = 0; InfIndex < InfluenceCountUV; InfIndex++)
			{
				LocalGTarget[InfIndex] += Row.Target * RowJacobian[InfIndex];
			}

			// Accumulate hessian
			for (int32 A = 0; A < InfluenceCountUV; A++)
			{
				const float ValueA = RowJacobian[A];
				for (int32 B = A; B < InfluenceCountUV; B++)
				{
					float ValueB = RowJacobian[B];
					LocalHessian[A * 16 + B] += ValueA * ValueB;
				}
			}
		}

		// Scatter gradient
		for (int32 InfIndex = 0; InfIndex < InfluenceCountUV; InfIndex++)
		{
			GData[GlobalIndices[InfIndex]] += LocalGTarget[InfIndex];
		}

		// Scatter upper triangle, bottom is mirrored through self-adjoint view later
		for (int32 A = 0; A < InfluenceCountUV; A++)
		{
			const int32 IdxA = GlobalIndices[A];
			for (int32 B = A; B < InfluenceCountUV; B++)
			{
				const int32 IdxB = GlobalIndices[B];
				HessianData[IdxB * HessianStride + IdxA] += LocalHessian[A * 16 + B];
			}
		}
	}
}

static void RegularizeHessianInPlace(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	const VEUV::FLocalSolveContext& LocalSolve,
	FResidualMatrix& Set,
	int32 UnknownCount)
{
	// Mirror the hessian, global scatter only applies it to the upper half
	Set.Hessian = Set.Hessian.selfadjointView<Eigen::Upper>();

	// Discrete Laplacian regularization
	if (Grid.Config.Solver.bEnableDiscreteLaplace)
	{
		float Alpha = 1e-3f;
		Eigen::Vector3f Extent = Grid.Voxels[0].LogicalMax - Grid.Voxels[0].LogicalMin;
		
		// TODO: This isn't correct, it can be non-uniform
		float FieldNudge = Alpha * std::sqrt(Extent.x());

		// TODO: This is a little bit ugly, written in haste
		for (int32 VoxelIdx : Region.VoxelIndices)
		{
			const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIdx];
			
			// No samples imply no allocated corners, in which case there's nothing to do
			if (Voxel.Samples.IsEmpty())
			{
				continue;
			}

			// Apply the discrete operator uniformly, just positive edges *should* be enough
			for (const VEUV::FVoxelEdge& Edge : VEUV::GVoxelLocalPositiveEdges)
			{
				int32 A = Grid.GetCornerIndex(Voxel.ID.x() + Edge.A.x(), Voxel.ID.y() + Edge.A.y(), Voxel.ID.z() + Edge.A.z());
				int32 B = Grid.GetCornerIndex(Voxel.ID.x() + Edge.B.x(), Voxel.ID.y() + Edge.B.y(), Voxel.ID.z() + Edge.B.z());

				// Get A corner
				if (!LocalSolve.CornerStates.IsEmpty())
				{
					if (const VEUV::FLocalCornerState* State = LocalSolve.CornerStates.Find(A))
					{
						A = State->bSolved ? INDEX_NONE : State->Mapping;
					}
					else
					{
						continue;
					}
				}

				// Get B corner
				if (!LocalSolve.CornerStates.IsEmpty())
				{
					if (const VEUV::FLocalCornerState* State = LocalSolve.CornerStates.Find(B))
					{
						B = State->bSolved ? INDEX_NONE : State->Mapping;
					}
					else
					{
						continue;
					}
				}

				// Must be either or
				check(A == INDEX_NONE || A < UnknownCount);
				check(B == INDEX_NONE || B < UnknownCount);

				// Apply nudge to all allocated U corners
				if (A != INDEX_NONE) { Set.Hessian(A * 2 + 0, A * 2 + 0) += FieldNudge; }
				if (B != INDEX_NONE) { Set.Hessian(B * 2 + 0, B * 2 + 0) += FieldNudge; }
				if (A != INDEX_NONE && B != INDEX_NONE) { Set.Hessian(A * 2 + 0, B * 2 + 0) -= FieldNudge; }
				if (A != INDEX_NONE && B != INDEX_NONE) { Set.Hessian(B * 2 + 0, A * 2 + 0) -= FieldNudge; }

				// Apply nudge to all allocated V corners
				if (A != INDEX_NONE) { Set.Hessian(A * 2 + 1, A * 2 + 1) += FieldNudge; }
				if (B != INDEX_NONE) { Set.Hessian(B * 2 + 1, B * 2 + 1) += FieldNudge; }
				if (A != INDEX_NONE && B != INDEX_NONE) { Set.Hessian(A * 2 + 1, B * 2 + 1) -= FieldNudge; }
				if (A != INDEX_NONE && B != INDEX_NONE) { Set.Hessian(B * 2 + 1, A * 2 + 1) -= FieldNudge; }
			}
		}
	}

	// Apply a gentle damping coefficient (Levenberg Marquardt)
	if (Grid.Config.Solver.bEnableDiagonalLMDamping)
	{
		float DiagNudge = Grid.Config.Solver.DiagonalNudgeAlpha * Set.Hessian.diagonal().mean();
		Set.Hessian.diagonal().array() += DiagNudge;
	}
}

static void RegularizeGradientInPlace(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	Eigen::VectorXf& Gradient)
{
	if (Grid.Config.Solver.bEnableDiscreteLaplace)
	{
		float Alpha = 1e-3f;
		Eigen::Vector3f Extent = Grid.Voxels[0].LogicalMax - Grid.Voxels[0].LogicalMin;
		
		// TODO: This isn't correct, it can be non-uniform
		float FieldNudge = Alpha * std::sqrt(Extent.x());

		for (int32 VoxelIdx : Region.VoxelIndices)
		{
			const VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIdx];
			
			// No samples imply no allocated corners, in which case there's nothing to do
			if (Voxel.Samples.IsEmpty())
			{
				continue;
			}

			for (const VEUV::FVoxelEdge& Edge : VEUV::GVoxelLocalPositiveEdges)
			{
				int32 A = Grid.GetCornerIndex(Voxel.ID.x() + Edge.A.x(), Voxel.ID.y() + Edge.A.y(), Voxel.ID.z() + Edge.A.z());
				int32 B = Grid.GetCornerIndex(Voxel.ID.x() + Edge.B.x(), Voxel.ID.y() + Edge.B.y(), Voxel.ID.z() + Edge.B.z());

				// Compared to second order, we need the solution for both edges
				if (A == INDEX_NONE || B == INDEX_NONE)
				{
					continue;
				}
				
				// U
				float DD_U = Region.SolvedUV[A * 2 + 0] - Region.SolvedUV[B * 2 + 0];
				Gradient[A * 2 + 0] += FieldNudge * DD_U;
				Gradient[B * 2 + 0] -= FieldNudge * DD_U;

				// V
				float DD_V = Region.SolvedUV[A * 2 + 1] - Region.SolvedUV[B * 2 + 1];
				Gradient[A * 2 + 1] += FieldNudge * DD_V;
				Gradient[B * 2 + 1] -= FieldNudge * DD_V;
			}
		}
	}

	// Analogous operation to Levenberg Marquardt, helps with bad conditioning and against norm explosion
	if (Grid.Config.Solver.bEnableDiagonalLMDamping)
	{
		Gradient += Grid.Config.Solver.DiagonalNudgeAlpha * Region.SolvedUV;
	}
}

static Eigen::VectorXf SolveFromResidualsInPlace(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	const VEUV::FLocalSolveContext& LocalSolve,
	FResidualMatrix& Set,
	int32 UnknownCount)
{
	RegularizeHessianInPlace(Grid, Region, LocalSolve, Set, UnknownCount);
	
	// Standard Cholesky
	Eigen::LLT<Eigen::MatrixXf> Cholesky;
	Cholesky.compute(Set.Hessian);
	return Cholesky.solve(Set.G);
}

static float ComputeSampleResidual78(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	VEUV::FLocalSolveContext& LocalSolve,
	const VEUV::FVoxelData& Voxel,
	bool bCaptureError)
{
	Eigen::Vector3f VoxelExtent = Voxel.LogicalMax - Voxel.LogicalMin;

	TArray<int32> ControlIndices;
	ControlIndices.SetNumUninitialized(Voxel.Samples.Num());
	
	// Preallocate control structures
	for (int32 i = 0; i < Voxel.Samples.Num(); i++)
	{
		ControlIndices[i] = LocalSolve.SparseResiduals.ControlOffset;
		AllocateControl(LocalSolve.SparseResiduals, 8, 6);
	}
	
	// Computed per sample
	ParallelFor(Voxel.Samples.Num(), [&](int32 SampleIndex)
	{
		const VEUV::FSampleData& Sample = Voxel.Samples[SampleIndex];
		
		Eigen::Vector3f NormPos = (Sample.Position - Voxel.LogicalMin).cwiseQuotient(VoxelExtent);

		VEUV::CheckFinite(NormPos);
		VEUV::CheckFinite(Sample.Normal);

		struct FUnknown
		{
			Eigen::Vector3f J_c;
			Eigen::Vector3f NxJc;
			Eigen::Vector3f JxNc;
			int32 Idx;
		};

		FUnknown Unknowns[8];
		int32 UnknownCount = 0;

		float R7xB = 0, R7yB = 0, R7zB = 0;
		float R8xB = 0, R8yB = 0, R8zB = 0;

		Eigen::Vector3f UVPD[8];
		VEUV::Geometry::GetTrilinearPartialXYZDerivatives(NormPos.x(), NormPos.y(), NormPos.z(), UVPD);

		// Evaluate against all corners
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);

					// Get corner lattice
					VEUV::FLocalCornerState Lattice;
					if (!LocalSolve.CornerStates.IsEmpty())
					{
						// May not be mapped
						const VEUV::FLocalCornerState* Found = LocalSolve.CornerStates.Find(CornerIdx);
						if (!Found)
						{
							continue;
						}
						
						Lattice = *Found;
						CornerIdx = Lattice.Mapping;
					}

					Eigen::Vector3f J_c  = UVPD[i * 4 + j * 2 + k].cwiseQuotient(VoxelExtent);
					Eigen::Vector3f NxJc = Sample.Normal.cross(J_c);
					Eigen::Vector3f JxNc = J_c.cross(Sample.Normal);

					VEUV::CheckFinite(J_c);
					VEUV::CheckFinite(NxJc);
					VEUV::CheckFinite(JxNc);

					// If the lattice is solved, we instead apply it on the target
					if (!Lattice.bSolved)
					{
						FUnknown& Unknown = Unknowns[UnknownCount++];
						Unknown.J_c = J_c;
						Unknown.NxJc = NxJc;
						Unknown.JxNc = JxNc;
						Unknown.Idx = CornerIdx;
					}
					else
					{
						float SolvedU = Region.SolvedUV(CornerIdx * 2 + 0);
						float SolvedV = Region.SolvedUV(CornerIdx * 2 + 1);

						VEUV::CheckFinite(SolvedU);
						VEUV::CheckFinite(SolvedV);

						R7xB -= NxJc.x() * SolvedU + (-J_c.x()) * SolvedV;
						R7yB -= NxJc.y() * SolvedU + (-J_c.y()) * SolvedV;
						R7zB -= NxJc.z() * SolvedU + (-J_c.z()) * SolvedV;

						R8xB -= (-J_c.x()) * SolvedU + JxNc.x() * SolvedV;
						R8yB -= (-J_c.y()) * SolvedU + JxNc.y() * SolvedV;
						R8zB -= (-J_c.z()) * SolvedU + JxNc.z() * SolvedV;
					}
				}
			}
		}
			
		// Overwrite the actual influence count
		VEUV::FSparseResidualRowControl& Control = LocalSolve.SparseResiduals.Controls[ControlIndices[SampleIndex]];
		Control.InfluenceCount = UnknownCount;

		// All solved?
		if (UnknownCount == 0)
		{
			return;
		}

		// Get rows
		VEUV::FSparseResidualRow& R7x = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 0];
		VEUV::FSparseResidualRow& R7y = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 1];
		VEUV::FSparseResidualRow& R7z = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 2];
		VEUV::FSparseResidualRow& R8x = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 3];
		VEUV::FSparseResidualRow& R8y = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 4];
		VEUV::FSparseResidualRow& R8z = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 5];

		// Assign targets
		R7x.Target = R7xB; 
		R7y.Target = R7yB;
		R7z.Target = R7zB;
		R8x.Target = R8xB; 
		R8y.Target = R8yB;
		R8z.Target = R8zB;

		VEUV::CheckFinite(R7xB);
		VEUV::CheckFinite(R7yB);
		VEUV::CheckFinite(R7zB);
		VEUV::CheckFinite(R8xB);
		VEUV::CheckFinite(R8yB);
		VEUV::CheckFinite(R8zB);

		// Accumulate all unknowns
		for (int32 Idx = 0; Idx < UnknownCount; Idx++)
		{
			const FUnknown& Unknown = Unknowns[Idx];
			LocalSolve.SparseResiduals.Indices[Control.InfluenceOffset + Idx] = Unknown.Idx;

			LocalSolve.SparseResiduals.JacobianResiduals[R7x.JacobianResidualOffset + Idx * 2 + 0] = Unknown.NxJc.x();
			LocalSolve.SparseResiduals.JacobianResiduals[R7x.JacobianResidualOffset + Idx * 2 + 1] = -Unknown.J_c.x();
			LocalSolve.SparseResiduals.JacobianResiduals[R7y.JacobianResidualOffset + Idx * 2 + 0] = Unknown.NxJc.y();
			LocalSolve.SparseResiduals.JacobianResiduals[R7y.JacobianResidualOffset + Idx * 2 + 1] = -Unknown.J_c.y();
			LocalSolve.SparseResiduals.JacobianResiduals[R7z.JacobianResidualOffset + Idx * 2 + 0] = Unknown.NxJc.z();
			LocalSolve.SparseResiduals.JacobianResiduals[R7z.JacobianResidualOffset + Idx * 2 + 1] = -Unknown.J_c.z();

			LocalSolve.SparseResiduals.JacobianResiduals[R8x.JacobianResidualOffset + Idx * 2 + 0] = -Unknown.J_c.x();
			LocalSolve.SparseResiduals.JacobianResiduals[R8x.JacobianResidualOffset + Idx * 2 + 1] = Unknown.JxNc.x();
			LocalSolve.SparseResiduals.JacobianResiduals[R8y.JacobianResidualOffset + Idx * 2 + 0] = -Unknown.J_c.y();
			LocalSolve.SparseResiduals.JacobianResiduals[R8y.JacobianResidualOffset + Idx * 2 + 1] = Unknown.JxNc.y();
			LocalSolve.SparseResiduals.JacobianResiduals[R8z.JacobianResidualOffset + Idx * 2 + 0] = -Unknown.J_c.z();
			LocalSolve.SparseResiduals.JacobianResiduals[R8z.JacobianResidualOffset + Idx * 2 + 1] = Unknown.JxNc.z();
		}
	});

	// Sort the control structure for sparse multiplications
	SortArray(LocalSolve.SparseResiduals);
	
	if (bCaptureError)
	{
		TArray<float> Errors;
		Errors.SetNumUninitialized(Voxel.Samples.Num());
		
		// Evaluate error on all samples
		ParallelFor(Voxel.Samples.Num(), [&](int32 i)
		{
			Errors[i] = VEUV::FSampling::EvaluateSampleErrorR78(Grid, Region, Voxel, Voxel.Samples[i]);
		});
		
		float ErrorSum = 0.0f;
		for (float Error : Errors)
		{
			ErrorSum += Error;
		}
		
		return ErrorSum;
	}
	
	return 0;
}

struct FImplicitGradients
{
	FImplicitGradients() = default;
	
	explicit FImplicitGradients(int32 Count)
	{
		G_R78 = Eigen::VectorXf::Zero(Count);
		G_R9ab = Eigen::VectorXf::Zero(Count);
		G_R10 = Eigen::VectorXf::Zero(Count);
	}
	
	FImplicitGradients& operator+=(const FImplicitGradients& Other)
	{
		G_R78 += Other.G_R78;
		T_R78 += Other.T_R78;
		
		if (Other.G_R9ab.size())
		{
			G_R9ab += Other.G_R9ab;
			T_R9ab += Other.T_R9ab;
		}
		
		if (Other.G_R10.size())
		{
			G_R10 += Other.G_R10;
			T_R10 += Other.T_R10;
		}
		
		SurfaceError += Other.SurfaceError;
		return *this;
	}
	
	void Reset()
	{
		G_R78.setZero();
		G_R9ab.setZero();
		G_R10.setZero();
		T_R78 = 0;
		T_R9ab = 0;
		T_R10 = 0;
		SurfaceError = 0;
	}
	
	Eigen::VectorXf G_R78;
	Eigen::VectorXf G_R9ab;
	Eigen::VectorXf G_R10;
	float T_R78 = 0;
	float T_R9ab = 0;
	float T_R10 = 0;
	float SurfaceError = 0;
};

static void CollectResidualSamples(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	TArray<FSampleResidualData>& Out)
{
	uint32 Count = 0;

	// Number of samples to allocate
	for (int32 Idx : Region.VoxelIndices)
	{
		Count += Grid.Voxels[Idx].Samples.Num();
	}
	
	Out.SetNumUninitialized(Count);
	
	uint32 Offset = 0;
	
	// Flatten all samples
	for (int32 Idx : Region.VoxelIndices)
	{
		const VEUV::FVoxelData& Voxel = Grid.Voxels[Idx];
		
		Eigen::Vector3f VoxelExtent = Voxel.LogicalMax - Voxel.LogicalMin;	
		
		for (const VEUV::FSampleData& Sample : Voxel.Samples)
		{
			FSampleResidualData& Data = Out[Offset++];
			Data.NormPosition = (Sample.Position - Voxel.LogicalMin).cwiseQuotient(VoxelExtent);
			Data.VoxelIndex = Idx;
			Data.Normal = Sample.Normal;
			Data.Type = Sample.Type;
		}
	}
}

struct FImplicitGradientConfig
{
	bool bWithR9ab = false;
	bool bWithR10 = false;
	bool bR10HardConstraint = false;
	bool bCaptureError = false;
};

static FImplicitGradients ImplicitGradientResidualR78_R9ab_R10(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	const TArray<FSampleResidualData>& Samples,
	TArray<FImplicitGradients>& TaskContexts,
	FImplicitGradientConfig Config)
{
	// Reset existing contexts
	for (FImplicitGradients& TaskContext : TaskContexts)
	{
		TaskContext.Reset();
	}
	
	// Computed per sample
	ParallelForWithExistingTaskContext(MakeArrayView(TaskContexts), Samples.Num(), 1024, [&](FImplicitGradients& Data, int32 SampleIndex)
	{
		const FSampleResidualData& Sample = Samples[SampleIndex];
		const VEUV::FVoxelData& Voxel = Grid.Voxels[Sample.VoxelIndex];
			
		// TODO: Precompute per voxel, should mostly fit in cache
		int32 CornerIndices[8];
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					int32 LocalIndex = i * 4 + j * 2 + k;
					CornerIndices[LocalIndex] = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				}
			}
		}
		
		Eigen::Vector3f VoxelExtent = Voxel.LogicalMax - Voxel.LogicalMin;	
	
		// Get partial derivatives
		Eigen::Vector3f UVPD[8];
		VEUV::Geometry::GetTrilinearPartialXYZDerivatives(Sample.NormPosition.x(), Sample.NormPosition.y(), Sample.NormPosition.z(), UVPD);

		// Get all jacobians, re-used numerous times
		Eigen::Vector3f J_C8[8];
		for (int32 i = 0; i < 8; i++)
		{
			J_C8[i] = UVPD[i].cwiseQuotient(VoxelExtent);
		}
		
		// Both R78 and R9ab can reuse Nabla_U/V
		Eigen::Vector3f Nabla_U = Eigen::Vector3f::Zero();
		Eigen::Vector3f Nabla_V = Eigen::Vector3f::Zero();

		// Accumulate against all corners
		for (int32 i = 0; i < 8; i++)
		{
			int32 CornerIndex = CornerIndices[i];
			
			const Eigen::Vector3f& J_C = J_C8[i];
			
			Eigen::Vector2f UV(Region.SolvedUV[CornerIndex * 2 + 0], Region.SolvedUV[CornerIndex * 2 + 1]);
			Nabla_U += UV.x() * J_C;
			Nabla_V += UV.y() * J_C;
		}
			
		// Certain samples may not have a valid frame, skip them for area weighting
		// This is a hard condition to "solve" for other than accepting that as the solution
		// space changes, certain samples may fall out of the valid manifold. Just keep solving
		// and we'll land in a good spot.
		const bool bValidFrame = Nabla_U.norm() > 1e-6f && Nabla_V.norm() > 1e-6f;
	
		// Optional error data, before orthonormalization
		if (Config.bCaptureError)
		{
			Data.SurfaceError += (Nabla_U.cross(Nabla_V) - Sample.Normal).squaredNorm();
		}
		
		VEUV::CheckFinite(Nabla_U);
		VEUV::CheckFinite(Nabla_V);

		// Forward, evaluate R78 energy terms
		Eigen::Vector3f R7 = Sample.Normal.cross(Nabla_U) - Nabla_V;
		Eigen::Vector3f R8 = -Nabla_U + Nabla_V.cross(Sample.Normal);
		
		// Forward, accumulated R9ab
		Eigen::Vector3f R9a;
		Eigen::Vector3f R9b;
			
		// R9ab temporaries
		Eigen::Vector3f U;
		Eigen::Vector3f V;
			
		// R10 barrier term
		float R10 = 0.0f;
			
		// R9ab & R10 samples only run per area sample, any other form of sample biasing requires voronoi area normalization or equiv.
		bool bAreaWeighted = bValidFrame && Sample.Type == EVEUVSampleType::Area;
		if (bAreaWeighted)
		{
			// Injectivity term
			if (Config.bWithR10)
			{
				// TODO: Should vary!
				float Eps = 1e-3f;
				float FoldWeight = 1.0f;
				
				// Injective regularization term
				// https://arxiv.org/pdf/2102.03069
				// X(D, e) = (D + sqrt(e^2 + D^2)) / 2
				
				// Log barrier term
				// https://web.stanford.edu/class/ee364a/lectures/barrier.pdf
				// R10(D, e) = -log(X(D, e))
				
				// Constraint term
				// R10(D, e) = 1/X
				
				// With that (S lazy hand)
				//  X'(D, E) = 
				//		S[ (D + sqrt(e^2 + D^2)) / 2 ]
				//		1/2 + S[ (e^2+D^2)^0.5 / 2 ]
				//		1/2 + 1/2 1/2 (e^2+D^2)^-0.5 S(e^2+D^2)
				//		1/2 + 1/2 1/2 (e^2+D^2)^-0.5 2 D
				//		1/2 + 1/2 (e^2+D^2)^-0.5 D
				//		1/2 (1 + D/sqrt(e^2+D^2))
				//
				// Naturally follows that
				//  R10'(D, e) = 
				//		S[ -log(X(D, e)) ]
				//      - 1/Y X'(D, e)
				//      - 1/Y ( 1/2 (1 + D/sqrt(e^2+D^2)) )
				//      - 1/Y ( 1/2 (D + sqrt(e^2+D^2)) / sqrt(e^2+D^2) )
				//      - 1/Y ( 1/2 ( 2Y / sqrt(e^2+D^2) ) )
				//      - 1/Y ( Y / sqrt(e^2+D^2) )
				//      - 1 / sqrt(e^2+D^2)
				//
				// And for the constraint term
				//  R10'(D, e) = 
				//      S[ 1/X(D, e) ]
				//      -Y^-2 S[X(D, e)]
				//      ...
				//      -Y^-2 ( Y / sqrt(e^2+D^2) )
				//      -1 / Y^2 Y/sqrt(e^2+D^2)
				//      -1 / Y^2 Y/sqrt(e^2+D^2)
				//      -1 / (Y sqrt(e^2+D^2))
				
				// TODO: max(small, ...), we have no need to reward/bias injective portions,
				//       we only need to penalize non-injective parts.
				
				float Determinant = Nabla_U.cross(Nabla_V).dot(Sample.Normal);
				
				// Constraint or log barrier?
				if (Config.bR10HardConstraint)
				{
					float SqD2E2 = std::sqrt(Determinant * Determinant + Eps * Eps);
					float Y = 0.5f * (Determinant + SqD2E2);
					float DenomDD = FMath::Max(1e-6f, Y * SqD2E2);
					R10 = - FoldWeight / DenomDD;
				}
				else
				{
					float Denom = FMath::Max(1e-6f, std::sqrt(Determinant * Determinant + Eps * Eps));
					R10 = - FoldWeight / Denom;
				}
				
				VEUV::CheckFinite(R10);
			}
			
			// Area term
			if (Config.bWithR9ab)
			{
				R9a = Eigen::Vector3f::Zero();
				R9b = Eigen::Vector3f::Zero();
				
				// Project onto tangent plane
				Eigen::Vector3f GProj_U = Nabla_U - Sample.Normal * Nabla_U.dot(Sample.Normal);
				Eigen::Vector3f GProj_V = Nabla_V - Sample.Normal * Nabla_V.dot(Sample.Normal);

				// Orthonormalize
				Eigen::Vector3f OrthT1 = GProj_U.normalized();
				Eigen::Vector3f OrthT2 = VEUV::Geometry::SafeOrthogonalize(OrthT1, GProj_V, Sample.Normal);

				VEUV::CheckNormalized(OrthT1);
				VEUV::CheckNormalized(OrthT2);

				// Effectively the strength of the term
				//   < 1, compresses the area to optimize others
				//   > 1, allocates more area to it
				float AreaPreservationRatio = 1.0f;
				U = std::sqrt(AreaPreservationRatio) * OrthT1;
				V = std::sqrt(AreaPreservationRatio) * OrthT2;

				// Flip if needed
				if (U.cross(V).dot(Sample.Normal) < 0.0f)
				{
					V = -V;
				}
				
				// Accumulate all unknowns
				for (int32 i = 0; i < 8; i++)
				{
					int32 CornerIndex = CornerIndices[i];
					
					const Eigen::Vector3f& J_c = J_C8[i];
			
					Eigen::Vector3f JxVc = J_c.cross(V);
					Eigen::Vector3f UxJc = U.cross(J_c);
					
					float SolvedU = Region.SolvedUV(CornerIndex * 2 + 0);
					float SolvedV = Region.SolvedUV(CornerIndex * 2 + 1);
					
					R9a += SolvedU * JxVc;
					R9b += SolvedV * UxJc;
				}
				
				R9a -= AreaPreservationRatio * Sample.Normal;
				R9b -= AreaPreservationRatio * Sample.Normal;

				VEUV::CheckFinite(R9a);
				VEUV::CheckFinite(R9b);
			}
		}
		
		// Backwards propagation
		for (int32 i = 0; i < 8; i++)
		{
			int32 CornerIndex = CornerIndices[i];
		
			const Eigen::Vector3f& J_C = J_C8[i];
			
			// J frame by normal
			Eigen::Vector3f NxJc = Sample.Normal.cross(J_C);
			Eigen::Vector3f JxNc = J_C.cross(Sample.Normal);
			
			// Accumulate R78 grad
			Data.G_R78[CornerIndex * 2 + 0] += NxJc.dot(R7) - J_C.dot(R8);
			Data.G_R78[CornerIndex * 2 + 1] += JxNc.dot(R8) - J_C.dot(R7);
			
			// Accumulate R78 trace
			Data.T_R78 += NxJc.squaredNorm() + J_C.squaredNorm();
			Data.T_R78 += J_C.squaredNorm() + JxNc.squaredNorm();
			
			if (bAreaWeighted)
			{
				if (Config.bWithR9ab)
				{
					Eigen::Vector3f JxVc = J_C.cross(V);
					Eigen::Vector3f UxJc = U.cross(J_C);
							
					// Accumulate R9ab grad
					Data.G_R9ab[CornerIndex * 2 + 0] += JxVc.dot(R9a);
					Data.G_R9ab[CornerIndex * 2 + 1] += UxJc.dot(R9b);
						
					// Accumulate R9ab trace
					Data.T_R9ab += JxVc.squaredNorm();
					Data.T_R9ab += UxJc.squaredNorm();
				}
				
				if (Config.bWithR10)
				{
					float DD_DU = J_C.cross(Nabla_V).dot(Sample.Normal);
					float DD_DV = Nabla_U.cross(J_C).dot(Sample.Normal);
					
					// Accumulate R10 grad
					Data.G_R10[CornerIndex * 2 + 0] += R10 * DD_DU;
					Data.G_R10[CornerIndex * 2 + 1] += R10 * DD_DV;
				
					// Accumulate R10 trace
					Data.T_R10 += DD_DU * DD_DU + DD_DV * DD_DV;
				}
			}
		}
	});
	
	FImplicitGradients ImplicitGradients(Region.SolvedUV.size());

	// Reduce task contexts
	for (const FImplicitGradients& Gradients : TaskContexts)
	{
		ImplicitGradients += Gradients;
	}
	
	return ImplicitGradients;
}

static float ComputeSampleResidual9(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FSolveRegion& Region,
	VEUV::FLocalSolveContext& LocalSolve,
	const VEUV::FVoxelData& Voxel,
	bool bCaptureError)
{
	Eigen::Vector3f VoxelExtent = Voxel.LogicalMax - Voxel.LogicalMin;
	
	// For R9 it's imperative that the spatial sample density correlates with area
	// Adaptive and complex samples are useful for conformity and orthorinality (R78),
	// but bias R9ab too much. So, just use area sampling.
	TArray<uint32> AreaSamples;
	AreaSamples.Reserve(Voxel.Samples.Num());
	
	// Filter all area samples
	for (int32 i = 0; i < Voxel.Samples.Num(); i++)
	{
		if (Voxel.Samples[i].Type == EVEUVSampleType::Area)
		{
			AreaSamples.Add(i);
		}
	}

	TArray<VEUV::FSparseResidualRowControl> Controls;
	Controls.SetNumUninitialized(AreaSamples.Num());
	
	TArray<float> Errors;
	if (bCaptureError)
	{
		Errors.SetNumUninitialized(AreaSamples.Num());
	}
	
	// Preallocate control structures
	for (int32 i = 0; i < AreaSamples.Num(); i++)
	{
		Controls[i] = AllocateControl(LocalSolve.SparseResiduals, 8, 6);
	}
	
	// Computed per area sample
	ParallelFor(AreaSamples.Num(), [&](int32 SampleIndex)
	{
		const VEUV::FSampleData& Sample = Voxel.Samples[AreaSamples[SampleIndex]];
			
		Eigen::Vector3f NormPos = (Sample.Position - Voxel.LogicalMin).cwiseQuotient(VoxelExtent);

		Eigen::Vector3f Nabla_U = Eigen::Vector3f::Zero();
		Eigen::Vector3f Nabla_V = Eigen::Vector3f::Zero();

		Eigen::Vector3f UVPD[8];
		VEUV::Geometry::GetTrilinearPartialXYZDerivatives(NormPos.x(), NormPos.y(), NormPos.z(), UVPD);
			
		// Evaluate against all corners
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);

					Eigen::Vector3f J_c = UVPD[i * 4 + j * 2 + k].cwiseQuotient(VoxelExtent);

					Eigen::Vector2f UV(Region.SolvedUV[CornerIdx * 2 + 0], Region.SolvedUV[CornerIdx * 2 + 1]);
					Nabla_U += UV.x() * J_c;
					Nabla_V += UV.y() * J_c;
				}
			}
		}
		
		// Optional error data
		if (bCaptureError)
		{
			Errors[SampleIndex] = (Nabla_U.cross(Nabla_V) - Sample.Normal).squaredNorm();
		}

		// Project onto tangent plane
		Nabla_U -= Sample.Normal * Nabla_U.dot(Sample.Normal);
		Nabla_V -= Sample.Normal * Nabla_V.dot(Sample.Normal);

		// Orthonormalize
		Eigen::Vector3f OrthT1 = Nabla_U.normalized();
		Eigen::Vector3f OrthT2 = VEUV::Geometry::SafeOrthogonalize(OrthT1, Nabla_V, Sample.Normal);

		// TODO: Why the ratio, there was a reason
		float Ratio = 1.0f;
		Eigen::Vector3f U = std::sqrt(Ratio) * OrthT1;
		Eigen::Vector3f V = std::sqrt(Ratio) * OrthT2;

		// Flip if needed
		if (U.cross(V).dot(Sample.Normal) < 0.0f)
		{
			V = -V;
		}

		VEUV::FSparseResidualRowControl& Control = Controls[SampleIndex];

		// Get rows
		VEUV::FSparseResidualRow& R9Ax = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 0];
		VEUV::FSparseResidualRow& R9Ay = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 1];
		VEUV::FSparseResidualRow& R9Az = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 2];
		VEUV::FSparseResidualRow& R9Bx = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 3];
		VEUV::FSparseResidualRow& R9By = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 4];
		VEUV::FSparseResidualRow& R9Bz = LocalSolve.SparseResiduals.Rows[Control.RowOffset + 5];

		// R9ab targets are the normals
		R9Ax.Target = Ratio * Sample.Normal.x();
		R9Ay.Target = Ratio * Sample.Normal.y();
		R9Az.Target = Ratio * Sample.Normal.z();
		R9Bx.Target = Ratio * Sample.Normal.x();
		R9By.Target = Ratio * Sample.Normal.y();
		R9Bz.Target = Ratio * Sample.Normal.z();

		int32 IndexOffset = 0;

		// Accumulate all unknowns
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);

					Eigen::Vector3f J_c  = UVPD[i * 4 + j * 2 + k].cwiseQuotient(VoxelExtent);
					Eigen::Vector3f JxVc = J_c.cross(V);
					Eigen::Vector3f UxJc = U.cross(J_c);

					LocalSolve.SparseResiduals.Indices[Control.InfluenceOffset + IndexOffset] = CornerIdx;

					LocalSolve.SparseResiduals.JacobianResiduals[R9Ax.JacobianResidualOffset + IndexOffset * 2 + 0] = JxVc.x();
					LocalSolve.SparseResiduals.JacobianResiduals[R9Ax.JacobianResidualOffset + IndexOffset * 2 + 1] = 0;
					LocalSolve.SparseResiduals.JacobianResiduals[R9Ay.JacobianResidualOffset + IndexOffset * 2 + 0] = JxVc.y();
					LocalSolve.SparseResiduals.JacobianResiduals[R9Ay.JacobianResidualOffset + IndexOffset * 2 + 1] = 0;
					LocalSolve.SparseResiduals.JacobianResiduals[R9Az.JacobianResidualOffset + IndexOffset * 2 + 0] = JxVc.z();
					LocalSolve.SparseResiduals.JacobianResiduals[R9Az.JacobianResidualOffset + IndexOffset * 2 + 1] = 0;

					LocalSolve.SparseResiduals.JacobianResiduals[R9Bx.JacobianResidualOffset + IndexOffset * 2 + 0] = 0;
					LocalSolve.SparseResiduals.JacobianResiduals[R9Bx.JacobianResidualOffset + IndexOffset * 2 + 1] = UxJc.x();
					LocalSolve.SparseResiduals.JacobianResiduals[R9By.JacobianResidualOffset + IndexOffset * 2 + 0] = 0;
					LocalSolve.SparseResiduals.JacobianResiduals[R9By.JacobianResidualOffset + IndexOffset * 2 + 1] = UxJc.y();
					LocalSolve.SparseResiduals.JacobianResiduals[R9Bz.JacobianResidualOffset + IndexOffset * 2 + 0] = 0;
					LocalSolve.SparseResiduals.JacobianResiduals[R9Bz.JacobianResidualOffset + IndexOffset * 2 + 1] = UxJc.z();

					IndexOffset++;
				}
			}
		}

		check(IndexOffset == Control.InfluenceCount);
	});

	// TODO: Should be able to pre-sort this key indices
	SortArray(LocalSolve.SparseResiduals);
	
	float ErrorSum = 0.0f;
	for (float Error : Errors)
	{
		ErrorSum += Error;
	}
	
	return ErrorSum;
}

static void CreateCornerMappings(
	const VEUV::FVoxelGrid& Grid,
	const TSet<int32>& SolvedCornerIndices,
	VEUV::FLocalSolveContext& LocalSolve,
	const VEUV::FVoxelData& Voxel)
{
	// Voxel has been accepted, create all corner states for region solves
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				if (LocalSolve.CornerStates.Contains(CornerIdx))
				{
					continue;
				}

				// Create the corner state
				VEUV::FLocalCornerState State;
				State.bSolved = SolvedCornerIndices.Contains(CornerIdx);
				State.Mapping = State.bSolved ? CornerIdx : LocalSolve.LocalCornerCount++;
				LocalSolve.CornerStates.Add(CornerIdx, State);
			}
		}
	}
}

static void MarkCornersAsSolved(
	const VEUV::FVoxelGrid& Grid,
	TSet<int32>& SolvedCornerIndices,
	const VEUV::FVoxelData& Voxel)
{
	// Mark all corners as solved
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				SolvedCornerIndices.Add(Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k));
			}
		}
	}
}

static bool HasAnyUnsolvedCorners(
	const VEUV::FVoxelGrid& Grid,
	const VEUV::FLocalSolveContext& LocalSolve,
	const VEUV::FVoxelData& Voxel)
{
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				const VEUV::FLocalCornerState* State = LocalSolve.CornerStates.Find(CornerIdx);
				if (State && !State->bSolved)
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

static void SolveLLSLocal(
	VEUV::FVoxelGrid& Grid,
	VEUV::FSolveRegion& Region,
	TSet<int32>& SolvedCornerIndices,
	const VEUV::FVoxelData& Voxel,
	const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	VEUV::FLocalSolveContext LocalSolve;

	// Preallocate the sparse residual states against the 3x3 solve region
	int32 NeighbourhoodSamples = 0;
	for (int32 z = -1; z < 2; z++)
	{
		for (int32 y = -1; y < 2; y++)
		{
			for (int32 x = -1; x < 2; x++)
			{
				Eigen::Vector3i ID = Voxel.ID + Eigen::Vector3i(x, y, z);
				
				// Bounds check
				if ((ID.array() < 0).any() ||
					ID.x() >= Grid.VoxelCount.X ||
					ID.y() >= Grid.VoxelCount.Y ||
					ID.z() >= Grid.VoxelCount.Z)
				{
					continue;
				}
				
				// Accumulate if allocated
				if (const int32* IdxPtr = Grid.VoxelIndexMap.Find(FIntVector(ID.x(), ID.y(), ID.z())))
				{
					NeighbourhoodSamples += Grid.Voxels[*IdxPtr].Samples.Num();
				}
			}
		}
	}
	
	// Preallocate with R78 influences and rows
	LocalSolve.SparseResiduals.Preallocate(NeighbourhoodSamples, 8, 6);
	
	float ErrorSum = 0;

	// Gather residuals from the 3x3x3 neighbourhood
	for (int32 z = -1; z < 2; z++)
	{
		for (int32 y = -1; y < 2; y++)
		{
			for (int32 x = -1; x < 2; x++)
			{
				Eigen::Vector3i NeighbourID = Voxel.ID + Eigen::Vector3i(x, y, z);

				// Bounds check
				if ((NeighbourID.array() < 0).any() ||
					NeighbourID.x() >= Grid.VoxelCount.X ||
					NeighbourID.y() >= Grid.VoxelCount.Y ||
					NeighbourID.z() >= Grid.VoxelCount.Z)
				{
					continue;
				}

				// Get index
				const int32* IdxPtr = Grid.VoxelIndexMap.Find(FIntVector(NeighbourID.x(), NeighbourID.y(), NeighbourID.z()));
				check(IdxPtr);

				// Create the corner mappings
				const VEUV::FVoxelData& Neighbour = Grid.Voxels[*IdxPtr];
				CreateCornerMappings(Grid, SolvedCornerIndices, LocalSolve, Neighbour);
				
				// If this is fully solved, just skip
				if (!HasAnyUnsolvedCorners(Grid, LocalSolve, Neighbour))
				{
					continue;
				}

				// Append all R78
				ErrorSum += ComputeSampleResidual78(Grid, Region, LocalSolve, Neighbour, Capture.IsValid());
			}
		}
	}

	if (Capture)
	{
		Capture->R78ErrorHistory.Add(ErrorSum);
	}
	
	// Anything to solve?
	if (LocalSolve.LocalCornerCount > 0)
	{
		if (!LocalSolve.SparseResiduals.Rows.IsEmpty())
		{
			// Create sparse set and scatter multiply
			FResidualMatrix Matrix = CreateResidualMatrix(LocalSolve.LocalCornerCount);
			MultiplyLocalResidualsWithGlobalScatter(LocalSolve.SparseResiduals, 0, Matrix);

			// Solve it
			Eigen::VectorXf LocalSolution = SolveFromResidualsInPlace(Grid, Region, LocalSolve, Matrix, LocalSolve.LocalCornerCount);

			// Inject the local solution back into the region, it's a substantially smaller problem space
			for (auto&& [GlobalIdx, State] : LocalSolve.CornerStates)
			{
				if (!State.bSolved)
				{
					Region.SolvedUV[GlobalIdx * 2 + 0] = LocalSolution[State.Mapping * 2 + 0];
					Region.SolvedUV[GlobalIdx * 2 + 1] = LocalSolution[State.Mapping * 2 + 1];
				}
			}
		}

		// Mark this voxel as solved
		MarkCornersAsSolved(Grid, SolvedCornerIndices, Voxel);
	}

	// Add index to solve region
	Region.VoxelIndices.Add(*Grid.VoxelIndexMap.Find(FIntVector(Voxel.ID.x(), Voxel.ID.y(), Voxel.ID.z())));
}

static void SeedCorner(
	VEUV::FVoxelGrid& Grid,
	VEUV::FSolveRegion& Region,
	TSet<int32>& SolvedCornerIndices,
	VEUV::FVoxelData& Voxel,
	int32 X, int32 Y, int32 Z,
	const Eigen::Vector2f& UV
	)
{
	int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + X, Voxel.ID.y() + Y, Voxel.ID.z() + Z);
	Region.SolvedUV[CornerIdx * 2 + 0] = UV.x();
	Region.SolvedUV[CornerIdx * 2 + 1] = UV.y();
	SolvedCornerIndices.Add(CornerIdx);
}

static void SetAndSolveSeedVoxel(
	VEUV::FVoxelGrid& Grid,
	VEUV::FSolveRegion& Region,
	TSet<int32>& SolvedCornerIndices,
	VEUV::FChartTraversalContext& Flood,
	const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	int32 Candidate        = 0;
	int32 CandidateSamples = 0;

	// Lazy pick the most complex voxel as a seed
	for (int32 I = 0; I < Grid.Voxels.Num(); I++)
	{
		if (Grid.Voxels[I].Samples.Num() > CandidateSamples)
		{
			CandidateSamples = Grid.Voxels[I].Samples.Num();
			Candidate = I;
		}
	}

	// Set seed voxel
	VEUV::FVoxelData& Voxel = Grid.Voxels[Candidate];
	Grid.SeedVoxelIndex = Candidate;

	// Seed two corners to pin the initial scale, orientation and translation (i.e., break symmetry)
	// This is the basis that the rest of the solution implicitly uses
	SeedCorner(Grid, Region, SolvedCornerIndices, Voxel, 0, 0, 0, Eigen::Vector2f(0, 0));
	SeedCorner(Grid, Region, SolvedCornerIndices, Voxel, 1, 0, 0, Eigen::Vector2f(1, 0));

	// Solve the initial local state (i.e., remaining corners against those seeded corners)
	SolveLLSLocal(Grid, Region, SolvedCornerIndices, Voxel, Capture);

	// Mark as done
	Voxel.bVisited = true;
	Flood.Visited.Add(Candidate);
}

void VEUV::FSolver::SolveContinuousField78(FVoxelGrid& Grid, FSolveRegion& Region, const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	FChartTraversalContext Flood;
	
	// Reset solution
	Region.SolvedUV = Eigen::VectorXf::Zero(Grid.CornerAllocator * 2);

	// Assign and solve seed voxel
	TSet<int32> SolvedCornerIndices;
	SetAndSolveSeedVoxel(Grid, Region, SolvedCornerIndices, Flood, Capture);

	// Solve on each traversal against its pinned states
	while (Flood.Visited.Num() + Flood.Skipped.Num() < Grid.Voxels.Num())
	{
		FTraversalInfo Info = FCharting::TraverseLowestCut(Grid, Flood);
		if (Info.From == INDEX_NONE)
		{
			// No immediate cut, just select a first available one
			Info.To = FCharting::FirstUnvisited(Grid, Flood);
			if (Info.To == INDEX_NONE)
			{
				break;
			}
		}

		// Solve progressive state
		FVoxelData& ToVoxel = Grid.Voxels[Info.To];
		SolveLLSLocal(Grid, Region, SolvedCornerIndices, ToVoxel, Capture);

		// Mark as visited
		ToVoxel.bVisited = true;
		Flood.Visited.Add(Info.To);
	}
}

static void SetSeedVoxel(VEUV::FVoxelGrid& Grid, VEUV::FSolveRegion& Region, TSet<int32>& SolvedCornerIndices)
{
	int32 Candidate        = 0;
	int32 CandidateSamples = 0;

	// Lazy pick the most complex voxel as a seed
	for (int32 i = 0; i < Grid.Voxels.Num(); i++)
	{
		if (Grid.Voxels[i].Samples.Num() > CandidateSamples)
		{
			CandidateSamples = Grid.Voxels[i].Samples.Num();
			Candidate = i;
		}
	}

	// Set seed voxel
	VEUV::FVoxelData& Voxel = Grid.Voxels[Candidate];
	Grid.SeedVoxelIndex = Candidate;

	// Seed two corners to pin the initial scale, orientation and translation (i.e., break symmetry)
	// This is the basis that the rest of the solution implicitly uses
	SeedCorner(Grid, Region, SolvedCornerIndices, Voxel, 0, 0, 0, Eigen::Vector2f(0, 0));
	SeedCorner(Grid, Region, SolvedCornerIndices, Voxel, 1, 0, 0, Eigen::Vector2f(1, 0));
}

void VEUV::FSolver::SolveContinuousDenseField78(FVoxelGrid& Grid, FSolveRegion& Region, const TSharedPtr<FDebugCapture>& Capture)
{
	// Reset solution
	Region.SolvedUV = Eigen::VectorXf::Zero(Grid.CornerAllocator * 2);

	// Seed voxel to break translation and orientation
	TSet<int32> SolvedCornerIndices;
	SetSeedVoxel(Grid, Region, SolvedCornerIndices);

	TArray<int32> CornerMappings;
	CornerMappings.Init(INDEX_NONE, Grid.CornerAllocator);

	// Create corner mappings
	int32 UnknownCount = 0;
	for (int32 i = 0; i < Grid.CornerAllocator; i++)
	{
		if (!SolvedCornerIndices.Contains(i))
		{
			CornerMappings[i] = UnknownCount++;
		}
	}

	FLocalSolveContext LocalSolve;
	LocalSolve.LocalCornerCount = UnknownCount;

	// Create corner states
	for (int32 i = 0; i < Grid.CornerAllocator; i++)
	{
		FLocalCornerState State;
		State.bSolved = SolvedCornerIndices.Contains(i);
		State.Mapping = State.bSolved ? i : CornerMappings[i];
		LocalSolve.CornerStates.Add(i, State);
	}

	// Number of samples
	int32 TotalSamples = 0;
	for (const FVoxelData& Voxel : Grid.Voxels)
	{
		TotalSamples += Voxel.Samples.Num();
	}

	// Preallocate control structures
	LocalSolve.SparseResiduals.Preallocate(TotalSamples, 8, 6);

	// Create residual-78 samples
	float ErrorSum = 0;
	for (int32 i = 0; i < Grid.Voxels.Num(); i++)
	{
		FVoxelData& Voxel = Grid.Voxels[i];
		
		if (!Voxel.Samples.IsEmpty())
		{
			ErrorSum += ComputeSampleResidual78(Grid, Region, LocalSolve, Voxel, Capture.IsValid());
			Region.VoxelIndices.Add(i);
		}
	}

	// Error history
	if (Capture)
	{
		Capture->R78ErrorHistory.Add(ErrorSum);
	}

	// Nothing to solve for?
	if (!UnknownCount || LocalSolve.SparseResiduals.Rows.IsEmpty())
	{
		return;
	}

	// Create sparse set and scatter multiply
	FResidualMatrix Matrix = CreateResidualMatrix(UnknownCount);
	MultiplyLocalResidualsWithGlobalScatter(LocalSolve.SparseResiduals, 0, Matrix);

	// Solve it
	Eigen::VectorXf Solution = SolveFromResidualsInPlace(Grid, Region, LocalSolve, Matrix, UnknownCount);

	// Inject the local solution back into the region
	for (auto&& [GlobalIdx, State] : LocalSolve.CornerStates)
	{
		if (!State.bSolved)
		{
			Region.SolvedUV[GlobalIdx * 2 + 0] = Solution[State.Mapping * 2 + 0];
			Region.SolvedUV[GlobalIdx * 2 + 1] = Solution[State.Mapping * 2 + 1];
		}
	}
}

static void SolveGlobalFieldHessianR9AB(VEUV::FVoxelGrid& Grid, VEUV::FSolveRegion& Region, int32 Iterations, const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	VEUV::FLocalSolveContext LocalSolve;
	LocalSolve.LocalCornerCount = Grid.CornerAllocator;

	// Count total samples for preallocation
	int32 TotalSamples = 0;
	for (int32 Idx : Region.VoxelIndices)
	{
		TotalSamples += Grid.Voxels[Idx].Samples.Num();
	}

	// Preallocate with R78 & R9ab influences and rows
	LocalSolve.SparseResiduals.Preallocate(TotalSamples * 2, 8, 6);

	// Within each voxel, append the R78 residuals
	// Note that these are invariant to the UV solution for global solves, so we can reuse them
	for (int32 Idx : Region.VoxelIndices)
	{
		ComputeSampleResidual78(Grid, Region, LocalSolve, Grid.Voxels[Idx], false);
	}

	// Create sparse and pre-multiply R78
	FResidualMatrix M_R78 = CreateResidualMatrix(LocalSolve.LocalCornerCount);
	MultiplyLocalResidualsWithGlobalScatter(LocalSolve.SparseResiduals, 0, M_R78);
	
	// Trace for energy norm
	float TraceR78 = M_R78.Hessian.trace();
	
	// Restore state for progressive solves
	VEUV::FSparseResidualArray::FSnapshot Snapshot = LocalSolve.SparseResiduals.Snapshot();
	
	// Standard scheduler
	VEUV::Schedulers::FAdam Scheduler;
	Scheduler.Init(LocalSolve.LocalCornerCount * 2);
	
	for (int32 Iteration = 0; Iteration < Iterations; Iteration++)
	{
		LocalSolve.SparseResiduals.Restore(Snapshot);

		// Within each voxel, append the R9ab residuals
		float ErrorSum = 0;
		for (int32 Idx : Region.VoxelIndices)
		{
			ErrorSum += ComputeSampleResidual9(Grid, Region, LocalSolve, Grid.Voxels[Idx], Capture.IsValid());
		}
		
		// Debug diagnostics
		if (Capture)
		{
			Capture->R9ErrorHistory.Add(ErrorSum);
		}

		// Create and construct separate R9ab hessian
		FResidualMatrix M_R9ab = CreateResidualMatrix(LocalSolve.LocalCornerCount);
		MultiplyLocalResidualsWithGlobalScatter(LocalSolve.SparseResiduals, Snapshot.ControlsCount, M_R9ab);
		
		// Safe R9ab trace
		float TraceR9ab = M_R9ab.Hessian.trace();
		if (TraceR9ab < FLT_EPSILON)
		{
			TraceR9ab = 1;
		}
		
		// R78 and R9ab have no canonical normalization, so, instead regularize the energies
		float TraceLambda = (TraceR78 / TraceR9ab) * Grid.Config.Solver.GlobalSolveR9abRegAlpha;
		
		// Combine both residuals with regularization
		FResidualMatrix M_Combined;
		M_Combined.Hessian = M_R78.Hessian + M_R9ab.Hessian * TraceLambda;
		M_Combined.G       = M_R78.G       + M_R9ab.G       * TraceLambda;
	
		// TODO: Regularize in outer once?
		RegularizeHessianInPlace(Grid, Region, LocalSolve, M_Combined, LocalSolve.LocalCornerCount);
	
		// Evaluate gradient
		Eigen::VectorXf Gradient = M_Combined.Hessian * Region.SolvedUV - M_Combined.G;
	
		// Diagnostic
		if (Capture)
		{
			Capture->R9GradNormHistory.Add(Gradient.norm());
		}
	
		// Apply scheduled step
		Eigen::VectorXf Delta = Scheduler.Step(Gradient, Grid.Config.Solver.GlobalSolveLR);
		Region.SolvedUV -= Delta;
	}
}

static void AddSnapshot(const VEUV::FVoxelGrid& Grid, VEUV::FSolveRegion& Region, const VEUV::FEigenMesh& Mesh, const FString& Name, const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	VEUV::FDebugCapture::FGeometrySnapshot& Snapshot = Capture->Snapshots.Emplace_GetRef();
	Snapshot.Name = Name;
	
	// Just keep the faces as is
	for (Eigen::Vector3i Face : Mesh.Faces)
	{
		Snapshot.Faces.Add(FInt32Vector3(Face.x(), Face.y(), Face.z()));
	}
	
	// Sample UVs in parallel
	Snapshot.VertexUVs.SetNumUninitialized(Mesh.Vertices.Num());
	ParallelFor(Mesh.Vertices.Num(), [&](int32 i)
	{
		Eigen::Vector2f UV = Grid.SampleUV(Region.SolvedUV, Mesh.Vertices[i]);
		Snapshot.VertexUVs[i] = FVector2f(UV.x(), UV.y());
	});

	// Normalize UVs to [0, 1]
	if (Snapshot.VertexUVs.Num() > 0)
	{
		FVector2f Min(FLT_MAX, FLT_MAX);
		FVector2f Max(-FLT_MAX, -FLT_MAX);
		
		for (const FVector2f& UV : Snapshot.VertexUVs)
		{
			Min.X = FMath::Min(Min.X, UV.X);
			Min.Y = FMath::Min(Min.Y, UV.Y);
			Max.X = FMath::Max(Max.X, UV.X);
			Max.Y = FMath::Max(Max.Y, UV.Y);
		}

		FVector2f Extent = Max - Min;
		FVector2f InvExtent(
			Extent.X > FLT_EPSILON ? 1.0f / Extent.X : 0.0f,
			Extent.Y > FLT_EPSILON ? 1.0f / Extent.Y : 0.0f
		);

		// Normalize in parallel (might not be worth it)
		ParallelFor(Snapshot.VertexUVs.Num(), [&](int32 i)
		{
			Snapshot.VertexUVs[i] = (Snapshot.VertexUVs[i] - Min) * InvExtent;
		});
	}
}

static int32 SolveGlobalFieldImplicitR10(VEUV::FVoxelGrid& Grid, VEUV::FSolveRegion& Region, const VEUV::FEigenMesh& Mesh, int32 MaxIterations, const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	TArray<FSampleResidualData> Samples;
	CollectResidualSamples(Grid, Region, Samples);
	
	// Worker task contexts
	TArray<FImplicitGradients> Tasks;
	Tasks.SetNum(FPlatformMisc::NumberOfCores());
	
	// Only care about 78 & 10
	for (FImplicitGradients& Task : Tasks) 
	{
		Task.G_R78 = Eigen::VectorXf::Zero(Region.SolvedUV.size());
		Task.G_R10 = Eigen::VectorXf::Zero(Region.SolvedUV.size());
	}
	
	VEUV::Schedulers::FAdam Scheduler;
	Scheduler.Init(Grid.CornerAllocator * 2);

	// Stop when gradient norm rate drops to 1% of peak rate
	VEUV::Schedulers::FNormSlopeCondition StopCondition;
	StopCondition.Threshold = 0.01f;
	
	// Only R10
	FImplicitGradientConfig Config;
	Config.bWithR10 = true;
	
	// Initial loss landscape is borderline pure noise
	int32 WarmupIterations = MaxIterations / 5;
	
	for (int32 Iteration = 0; Iteration < MaxIterations; Iteration++)
	{
		FImplicitGradients ImplicitGradients = ImplicitGradientResidualR78_R9ab_R10(
			Grid, Region, Samples,
			Tasks, Config
		);

		// Safe traces
		ImplicitGradients.T_R10  = ImplicitGradients.T_R10 < FLT_EPSILON ? 1.0f : ImplicitGradients.T_R10;
		
		// R78 and R10 have no canonical normalization, so, instead regularize the energies
		float TraceR10Lambda = (ImplicitGradients.T_R78 / ImplicitGradients.T_R10 ) * Grid.Config.Solver.GlobalSolveR10RegAlpha;
		
		// Combine both gradients with regularization
		Eigen::VectorXf Gradient = ImplicitGradients.G_R78 + ImplicitGradients.G_R10 * TraceR10Lambda;

		// Bias/regularize the gradients to that of a continuous field
		RegularizeGradientInPlace(Grid, Region, Gradient);

		// Diagnostic
		if (Capture)
		{
			Capture->R10GradNormHistory.Add(Gradient.norm());
			if (Capture->bWithSnapshots)
			{
				AddSnapshot(Grid, Region, Mesh, FString::Printf(TEXT("R78-R10-%i"), Iteration), Capture);
			}
		}
	
		// Apply scheduled step
		Eigen::VectorXf Delta = Scheduler.Step(Gradient, Grid.Config.Solver.GlobalSolveLR);
		Region.SolvedUV -= Delta;

		// Early out if gradient norm plateaued (after warmup)
		if (Iteration > WarmupIterations && StopCondition.Step(Gradient.norm()))
		{
			return Iteration + 1;
		}
	}

	return MaxIterations;
}

static int32 SolveGlobalFieldImplicitR9AB(
	VEUV::FVoxelGrid& Grid, 
	VEUV::FSolveRegion& Region,
	VEUV::Schedulers::FAdam::FCoeffInvariantSnapshot& Snapshot,
	const VEUV::FEigenMesh& Mesh,
	int32 MaxIterations,
	bool bIsInitialSolve,
	bool bForceInjectivity,
	const TSharedPtr<VEUV::FDebugCapture>& Capture)
{
	// Pre-condition the solution to the nearest *linear* minima, after which
	// GD handles the non-linear dynamics.
	if (bIsInitialSolve && Grid.Config.Solver.bGlobalSolveEnableLinearPrecond)
	{
		SolveGlobalFieldHessianR9AB(Grid, Region, 1, Capture);
	}
	
	// Worker task contexts
	TArray<FImplicitGradients> Tasks;
	Tasks.SetNum(FPlatformMisc::NumberOfCores());
	
	// Init all terms
	for (FImplicitGradients& Task : Tasks) 
	{
		Task.G_R78 = Eigen::VectorXf::Zero(Region.SolvedUV.size());
		Task.G_R9ab = Eigen::VectorXf::Zero(Region.SolvedUV.size());
		Task.G_R10 = Eigen::VectorXf::Zero(Region.SolvedUV.size());
	}

	TArray<FSampleResidualData> Samples;
	CollectResidualSamples(Grid, Region, Samples);
	
	// Standard scheduler
	VEUV::Schedulers::FAdam Scheduler;
	Scheduler.Init(Grid.CornerAllocator * 2);
	
	// Refinement solves must keep the momentum
	if (!bIsInitialSolve)
	{
		Scheduler.SetFromSnapshot(Snapshot);
	}
	
	// Stop when surface error rate drops to 1% of peak rate
	VEUV::Schedulers::FNormSlopeCondition StopCondition;
	StopCondition.Threshold = 0.01f;
	
	// Initial loss landscape is borderline pure noise
	int32 WarmupIterations = bIsInitialSolve ? MaxIterations / 5 : 0;
	
	// Solve for both R9ab and R10
	FImplicitGradientConfig Config;
	Config.bWithR9ab = true;
	Config.bWithR10 = Grid.Config.Solver.bEnableInjectiveTerm;
	Config.bR10HardConstraint = bForceInjectivity;
	Config.bCaptureError = Capture.IsValid();
	
	int32 Iteration = 0;
	for (; Iteration < MaxIterations; Iteration++)
	{
		FImplicitGradients ImplicitGradients = ImplicitGradientResidualR78_R9ab_R10(
			Grid, Region, Samples,
			Tasks, Config
		);
		
		// Debug diagnostics
		if (Capture)
		{
			Capture->R9ErrorHistory.Add(ImplicitGradients.SurfaceError);
			if (Capture->bWithSnapshots)
			{
				AddSnapshot(Grid, Region, Mesh, FString::Printf(TEXT("R78-R9AB-R10-%i"), Iteration), Capture);
			}
		}

		// Safe traces
		ImplicitGradients.T_R9ab = ImplicitGradients.T_R9ab < FLT_EPSILON ? 1.0f : ImplicitGradients.T_R9ab;
		ImplicitGradients.T_R10  = ImplicitGradients.T_R10 < FLT_EPSILON ? 1.0f : ImplicitGradients.T_R10;
		
		// R78 and R9ab have no canonical normalization, so, instead regularize the energies
		float TraceR9Lambda  = (ImplicitGradients.T_R78 / ImplicitGradients.T_R9ab) * Grid.Config.Solver.GlobalSolveR9abRegAlpha;
		float TraceR10Lambda = (ImplicitGradients.T_R78 / ImplicitGradients.T_R10 ) * Grid.Config.Solver.GlobalSolveR10RegAlpha;
		
		// Combine both gradients with regularization
		Eigen::VectorXf Gradient = 
			ImplicitGradients.G_R78 +
			ImplicitGradients.G_R9ab * TraceR9Lambda +
			ImplicitGradients.G_R10 * TraceR10Lambda;

		// Bias/regularize the gradients to that of a continuous field
		RegularizeGradientInPlace(Grid, Region, Gradient);

		// Diagnostic
		if (Capture)
		{
			Capture->R9GradNormHistory.Add(Gradient.norm());
		}
	
		// Apply scheduled step
		Eigen::VectorXf Delta = Scheduler.Step(Gradient, Grid.Config.Solver.GlobalSolveLR);
		Region.SolvedUV -= Delta;

		// Early out if surface error plateaued (after warmup)
		if (Iteration > WarmupIterations && StopCondition.Step(ImplicitGradients.SurfaceError))
		{
			break;
		}
	}
	
	// If initial solve, extract the momentum
	if (bIsInitialSolve)
	{
		Snapshot = Scheduler.GetSnapshot();
	}

	return Iteration;
}

int32 VEUV::FSolver::SolveGlobalFieldR10(FVoxelGrid& Grid, FSolveRegion& Region, const FEigenMesh& Mesh, int32 MaxIterations, const TSharedPtr<FDebugCapture>& Capture)
{
	if (Grid.Config.Solver.bEnableInjectiveTerm)
	{
		return SolveGlobalFieldImplicitR10(Grid, Region, Mesh, MaxIterations, Capture);
	}
	
	return 0;
}

int32 VEUV::FSolver::SolveGlobalFieldR9AB(
	FVoxelGrid& Grid, 
	FSolveRegion& Region,
	Schedulers::FAdam::FCoeffInvariantSnapshot& Snapshot,
	const VEUV::FEigenMesh& Mesh,
	int32 MaxIterations, 
	bool bIsInitialSolve,
	bool bForceInjectivity,
	const TSharedPtr<FDebugCapture>& Capture)
{
	// Solving for R78 is usually fine, it's global state doesn't change too much.
	// However, while we can "solve" for the R9ab global minimum, simply sweeping to
	// said minimum is incorrect since the gradient landscape changes drastically
	// along the sweep. So, we instead do some standard gradient descent to target.
	
	if (Grid.Config.Solver.bGlobalSolveGDFirstOrder)
	{
		return SolveGlobalFieldImplicitR9AB(Grid, Region, Snapshot, Mesh, MaxIterations, bIsInitialSolve, bForceInjectivity, Capture);
	}
	else
	{
		SolveGlobalFieldHessianR9AB(Grid, Region, MaxIterations, Capture);
		return MaxIterations;
	}
}

VEUV::FUVNormalization VEUV::FSolver::GetNormalization(const FVoxelGrid& Grid, FSolveRegion& Region)
{	
	FUVNormalization Out;
	
	int32 CornerCount = Region.SolvedUV.size() / 2;
	
	// Active state mask
	Out.CornerMasks.SetNum(CornerCount, false);
	
	for (int32 VoxelIdx : Region.VoxelIndices)
	{
		const FVoxelData& Voxel = Grid.Voxels[VoxelIdx];
		if (Voxel.Samples.IsEmpty())
		{
			continue;
		}

		// Mark all allocated corners
		for (int32 i = 0; i < 2; i++)
		{
			for (int32 j = 0; j < 2; j++)
			{
				for (int32 k = 0; k < 2; k++)
				{
					int32 CornerIdx = Grid.GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
					if (CornerIdx != INDEX_NONE && CornerIdx < CornerCount)
					{
						Out.CornerMasks[CornerIdx] = true;
					}
				}
			}
		}
	}

	float MinU = FLT_MAX, MaxU = -FLT_MAX;
	float MinV = FLT_MAX, MaxV = -FLT_MAX;

	// Min max the solution
	for (int32 i = 0; i < CornerCount; i++)
	{
		if (Out.CornerMasks[i])
		{
			MinU = std::min(MinU, Region.SolvedUV[i * 2 + 0]);
			MaxU = std::max(MaxU, Region.SolvedUV[i * 2 + 0]);
			MinV = std::min(MinV, Region.SolvedUV[i * 2 + 1]);
			MaxV = std::max(MaxV, Region.SolvedUV[i * 2 + 1]);
		}
	}

	float RangeU = MaxU - MinU;
	float RangeV = MaxV - MinV;
	
	// Ignore degenerate solutions
	if (RangeU < FLT_EPSILON || RangeV < FLT_EPSILON)
	{
		return {};
	}

	// Keep normalization uniform,  important
	float MaxRange = FMath::Max(RangeU, RangeV);
	
	Out.MinU = MinU;
	Out.MinV = MinV;
	Out.UniformScale = MaxRange;
	return Out;
}

void VEUV::FSolver::Normalize(const FVoxelGrid& Grid, FSolveRegion& Region)
{
	FUVNormalization Norm = GetNormalization(Grid, Region);

	// GetNormalization returns a default-constructed value (empty CornerMasks) when
	// the UV solution is degenerate; skip normalization rather than index into it.
	if (Norm.CornerMasks.Num() == 0)
	{
		return;
	}

	// Normalize all relevant states
	for (int32 i = 0; i < Region.SolvedUV.size() / 2; i++)
	{
		if (Norm.CornerMasks[i])
		{
			Region.SolvedUV[i * 2 + 0] = (Region.SolvedUV[i * 2 + 0] - Norm.MinU) / Norm.UniformScale;
			Region.SolvedUV[i * 2 + 1] = (Region.SolvedUV[i * 2 + 1] - Norm.MinV) / Norm.UniformScale;
		}
		else
		{
			Region.SolvedUV[i * 2 + 0] = 0.0f;
			Region.SolvedUV[i * 2 + 1] = 0.0f;
		}
	}
}

#if VEUV_VALIDATE
UE_ENABLE_OPTIMIZATION
#endif // VEUV_VALIDATE
