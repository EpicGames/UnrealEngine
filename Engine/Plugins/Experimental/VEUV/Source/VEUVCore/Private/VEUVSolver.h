// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVVoxelGrid.h"
#include "VEUVEigen.h"
#include "VEUVGeometry.h"
#include "VEUVSchedulers.h"
#include "VEUV/VEUVDebugCapture.h"

namespace VEUV
{
	struct FSparseResidualRow
	{
		/** Starting offset for jacobians */
		int32 JacobianResidualOffset = 0;
		
		/** Target (B) RHS value for LS-solve */
		float Target = 0.0f;
	};

	struct FSparseResidualRowControl
	{
		/** Total number of influences this control has (i.e., corners) */
		uint8 InfluenceCount = 0;
		
		/** Total number of rows for this control */
		uint8 RowCount = 0;
		
		/** Starting offsets */
		int32 InfluenceOffset = 0;
		int32 RowOffset = 0;
	};

	struct FSparseResidualArray
	{
		struct FSnapshot
		{
			int32 ControlsCount, RowsCount, JacobiansCount, IndicesCount;
		};

		FSnapshot Snapshot() const
		{
			return { ControlOffset, RowOffset, JacobianOffset, IndexOffset };
		}

		void Restore(const FSnapshot& Snapshot)
		{
			ControlOffset = Snapshot.ControlsCount;
			RowOffset = Snapshot.RowsCount;
			JacobianOffset = Snapshot.JacobiansCount;
			IndexOffset = Snapshot.IndicesCount;
		}

		void Preallocate(int32 NumControls, int32 MaxInfluencesPerControl, int32 RowsPerControl)
		{
			if (int32 Num = ControlOffset + NumControls; Num > Controls.Num())
			{
				Controls.SetNumUninitialized(Num);
			}
			
			if (int32 Num = RowOffset + NumControls * RowsPerControl; Num > Rows.Num())
			{
				Rows.SetNumUninitialized(Num);
			}
			
			if (int32 Num = IndexOffset + NumControls * MaxInfluencesPerControl; Num > Indices.Num())
			{
				Indices.SetNumUninitialized(Num);
			}
			
			if (int32 Num = JacobianOffset + NumControls * MaxInfluencesPerControl * RowsPerControl * 2; Num > JacobianResiduals.Num())
			{
				JacobianResiduals.SetNumUninitialized(Num);
			}
		}

		/** Current allocation offsets */
		int32 ControlOffset = 0;
		int32 RowOffset = 0;
		int32 JacobianOffset = 0;
		int32 IndexOffset = 0;

		/** Payloads */
		TArray<FSparseResidualRowControl> Controls;
		TArray<FSparseResidualRow> Rows;
		TArray<float> JacobianResiduals;
		TArray<int32> Indices;
	};

	struct FLocalCornerState
	{
		/** If solved, this corner is effectively pinned */
		bool bSolved = false;
		
		/** The corner mapping assigned */
		int32 Mapping = 0;
	};

	struct FLocalSolveContext
	{
		/** All residuals, influences are the corners */
		FSparseResidualArray SparseResiduals;
		
		/** Total number of corners allocated */
		int32 LocalCornerCount = 0;
		
		/** All corners */
		TMap<int32, FLocalCornerState> CornerStates;
	};

	struct FSolveRegion
	{
		/** Solved UVs per corners, [U0, V0, U1, V1, ...] */
		Eigen::VectorXf SolvedUV;
		
		/** All voxels considered */
		TArray<int32> VoxelIndices;
	};
	
	struct FUVNormalization
	{
		Eigen::Vector2f Transform(const Eigen::Vector2f& UV) const
		{
			return Eigen::Vector2f(
				(UV.x() - MinU) / UniformScale,
				(UV.y() - MinV) / UniformScale
			);
		}
		
		float MinU = 0.0f;
		float MinV = 0.0f;
		float UniformScale = 1.0f;
		TBitArray<> CornerMasks;
	};

	class FSolver
	{
	public:
		/**
		 * Solve the continuous parameterization with the R78 residuals.
		 *	Progressively seeds one voxel then solves each subsequent voxel against
		 *	already-pinned neighbours in a 3x3x3 neighbourhood LLS.
		 * @param Grid Parent grid
		 * @param Region Region to solve for
		 * @param Capture Optional, debug capture
		 */
		static void SolveContinuousField78(
			FVoxelGrid& Grid,
			FSolveRegion& Region, 
			const TSharedPtr<FDebugCapture>& Capture
		);

		/**
		 * Solve the continuous parameterization with the R78 residuals.
		 *	Globally solves voxels against initial seed set, instead of 3x3x3 LLS.
		 * @param Grid Parent grid
		 * @param Region Region to solve for
		 * @param Capture Optional, debug capture
		 */
		static void SolveContinuousDenseField78(
			FVoxelGrid& Grid,
			FSolveRegion& Region,
			const TSharedPtr<FDebugCapture>& Capture
		);
		
		/**
		 * Solve the global parameterization with the R10 residuals
		 * ? Internally this computes R78 on top of R10
		 * @param Grid Parent grid
		 * @param Region Region to solve for
		 * @param MaxIterations Maximum number of iterations
		 * @param Capture Optional, debug capture
		 */
		static int32 SolveGlobalFieldR10(
			FVoxelGrid& Grid,
			FSolveRegion& Region,
			const FEigenMesh& Mesh,
			int32 MaxIterations,
			const TSharedPtr<FDebugCapture>& Capture
		);
		
		/**
		 * Solve the global parameterization with the R9 residuals
		 * ? Internally this computes R78 on top of R9
		 * @param Grid Parent grid
		 * @param Region Region to solve for
		 * @param MaxIterations Maximum number of iterations
		 * @param Capture Optional, debug capture
		 */
		static int32 SolveGlobalFieldR9AB(
			FVoxelGrid& Grid,
			FSolveRegion& Region,
			Schedulers::FAdam::FCoeffInvariantSnapshot& Snapshot,
			const VEUV::FEigenMesh& Mesh,
			int32 MaxIterations,
			bool bIsInitialSolve,
			bool bForceInjectivity,
			const TSharedPtr<FDebugCapture>& Capture
			);

		/**
		* Normalize the produced UV solution
		 * @param Grid Parent grid
		 * @param Region Region to normalize for
		 */
		static FUVNormalization GetNormalization(
			const FVoxelGrid& Grid,
			FSolveRegion& Region
		);

		/**
		* Normalize the produced UV solution
		 * @param Grid Parent grid
		 * @param Region Region to normalize for
		 */
		static void Normalize(
			const FVoxelGrid& Grid,
			FSolveRegion& Region
		);
	};
}
