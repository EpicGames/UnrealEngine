// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVSplitMesh.h"
#include "VEUVGeometry.h"
#include "VEUVCharting.h"
#include "Async/ParallelFor.h"

struct FRegionRemap
{
	TArray<int32> VertexRemap;
};

struct FRemapContext
{
	void Initialize(int32 ChartCount, int32 VertexCount)
	{
		Remaps.SetNum(ChartCount);
		
		DefaultRemap.VertexRemap.Init(INDEX_NONE, VertexCount);
		
		for (FRegionRemap& Remap : Remaps)
		{
			Remap.VertexRemap.Init(INDEX_NONE, VertexCount);
		}
	}
	
	FRegionRemap& GetDefaultRemap()
	{
		return DefaultRemap;
	}
	
	FRegionRemap& Get(int32 Index)
	{
		return Remaps[Index];
	}
	
private:
	TArray<FRegionRemap> Remaps;
	
	FRegionRemap DefaultRemap;
};

enum class EFaceClassification
{
	/** Face belongs to a single boundary */
	Singular,
	
	/** Face requires splitting */
	Split
};

struct FFaceClassification
{
	EFaceClassification Result;
	int32 Value;
};

static void SplitFacesInternal(
	VEUV::FVoxelGrid& Grid,
	const VEUV::FEigenMesh& InMesh,
	VEUV::FEigenMesh& OutMesh,
	bool bIsVoxelSplit,
	const VEUV::FGlobalSolveContext* SolveContext,
	TArray<int32>* OutVertexVoxelIndices,
	TArray<VEUV::FVertexSource>* OutVertexSources)
{
	FRemapContext RemapContext;
	
	// Initialize remap for reuse and welding
	if (bIsVoxelSplit)
	{
		RemapContext.Initialize(Grid.Voxels.Num(), InMesh.Vertices.Num());
	}
	else
	{
		RemapContext.Initialize(SolveContext->Charts.Num(), InMesh.Vertices.Num()); //-V522
	}
	
	// Helper, get the remap for a voxel index
	auto GetRemap = [&](int32 VoxelIndex) -> FRegionRemap&
	{
		if (bIsVoxelSplit)
		{
			return RemapContext.Get(VoxelIndex);
		}
		else
		{
			// If charted, reuse the remap for the whole chart (i.e., weld within it)
			int32 ChartIndex = SolveContext->VoxelChartIndices[VoxelIndex];
			if (ChartIndex == INDEX_NONE)
			{
				return RemapContext.GetDefaultRemap();
			}
			
			return RemapContext.Get(ChartIndex);
		}
	};

	// Helper, pulls a vertex and its relevant attributes
	auto PullVertex = [&](FRegionRemap& Remap, int32 SourceIndex, int32 VoxelIndex) -> int32
	{
		// If remapped, just reuse
		if (Remap.VertexRemap[SourceIndex] != INDEX_NONE)
		{
			return Remap.VertexRemap[SourceIndex];
		}

		// Set remap
		int32 Index = OutMesh.Vertices.Num();
		Remap.VertexRemap[SourceIndex] = Index;
			
		// Append attributes
		OutMesh.Vertices.Add(InMesh.Vertices[SourceIndex]);
			
		// Append voxel association
		if (OutVertexVoxelIndices)
		{
			OutVertexVoxelIndices->Add(VoxelIndex);
		}
		
		// Append identity vertex source
		if (OutVertexSources)
		{
			VEUV::FVertexSource Src;
			Src.Vertices[0] = SourceIndex;
			Src.Weights[0] = 1.0f;
			OutVertexSources->Add(Src);
		}
		
		return Index;
	};
	
	TArray<FFaceClassification> Classifications;
	Classifications.SetNumUninitialized(InMesh.Faces.Num());
	
	// Classify in parallel
	ParallelFor(InMesh.Faces.Num(), [&](int32 FaceIndex)
	{
		FFaceClassification& Class = Classifications[FaceIndex];
			
		const Eigen::Vector3i& Face = InMesh.Faces[FaceIndex];
		const Eigen::Vector3f& P0 = InMesh.Vertices[Face.x()];
		const Eigen::Vector3f& P1 = InMesh.Vertices[Face.y()];
		const Eigen::Vector3f& P2 = InMesh.Vertices[Face.z()];
			
		// Find all overlapping voxels
		TArray<int32, TInlineAllocator<32>> OverlappingVoxels;
		Grid.FindOverlappingVoxels(P0, P1, P2, OverlappingVoxels);

		// Guard against degenerate case of triangle that somehow overlapped no voxels
		if (OverlappingVoxels.IsEmpty())
		{
			Class.Result = EFaceClassification::Split;
			Class.Value  = INDEX_NONE;
			return;
		}

		// If a single one, just append it
		if (OverlappingVoxels.Num() == 1)
		{
			Class.Result = EFaceClassification::Singular;
			Class.Value  = OverlappingVoxels[0];
			return;
		}

		// Just because we have multiple voxels doesn't mean they aren't a single chart
		if (!bIsVoxelSplit && SolveContext)
		{
			bool bSameChart = true;
			
			// Assume its first chart
			int32 BaseChart = SolveContext->VoxelChartIndices.IsValidIndex(OverlappingVoxels[0]) ? SolveContext->VoxelChartIndices[OverlappingVoxels[0]] : INDEX_NONE;

			// Do a linear check on all subsequent voxels
			for (int32 i = 1; i < OverlappingVoxels.Num(); i++)
			{
				int32 Chart = SolveContext->VoxelChartIndices.IsValidIndex(OverlappingVoxels[i]) ? SolveContext->VoxelChartIndices[OverlappingVoxels[i]] : INDEX_NONE;
				if (Chart != BaseChart)
				{
					bSameChart = false;
					break;
				}
			}

			// If the same, accept as is
			if (bSameChart)
			{
				Class.Result = EFaceClassification::Singular;
				Class.Value  = OverlappingVoxels[0];
				return;
			}
		}
			
		// Otherwise, split it
		Class.Result = EFaceClassification::Split;
		Class.Value  = INDEX_NONE;
	});
	
	TArray<int32, TInlineAllocator<32>> OverlappingVoxels;
	TArray<Eigen::Vector3f> ClippedVertices;

	// Visit all faces
	for (int32 FaceIndex = 0; FaceIndex < InMesh.Faces.Num(); FaceIndex++)
	{
		FFaceClassification& Class = Classifications[FaceIndex];
		
		const Eigen::Vector3i& Face = InMesh.Faces[FaceIndex];
		const Eigen::Vector3f& P0 = InMesh.Vertices[Face.x()];
		const Eigen::Vector3f& P1 = InMesh.Vertices[Face.y()];
		const Eigen::Vector3f& P2 = InMesh.Vertices[Face.z()];
		
		// If singular, just accept as is
		if (Class.Result == EFaceClassification::Singular)
		{
			FRegionRemap& Remap = GetRemap(Class.Value);
			
			// Create association
			if (bIsVoxelSplit && Class.Value != INDEX_NONE)
			{
				VEUV::FVoxelData& Voxel = Grid.Voxels[Class.Value];
				Voxel.Faces.Add(OutMesh.Faces.Num());
			}
			
			// Assign voxel per-vertex so AssignVertexUVs interpolates the chart cage rather 
			// than extrapolating Class.Value's cage when the triangle straddles voxels.
			auto VertexVoxel = [&Class, &Grid, SolveContext, bIsVoxelSplit](const Eigen::Vector3f& Pos) -> int32
			{
				// Single-voxel case: nothing to disambiguate
				if (bIsVoxelSplit || Class.Value == INDEX_NONE)
				{
					return Class.Value;
				}

				// Multi-voxel case: Assign by snapping to voxel, but guard against edge case
				// where that voxel wound up in a different chart 
				// (e.g. due to floating point error causing the tri-box overlap test to disagree with GetVoxelIndexAtPosition)
				const int32 PosVoxel = Grid.GetVoxelIndexAtPosition(Pos);
				if (SolveContext)
				{
					const int32 ClassChart = SolveContext->VoxelChartIndices.IsValidIndex(Class.Value) ? SolveContext->VoxelChartIndices[Class.Value] : INDEX_NONE;
					const int32 PosChart = SolveContext->VoxelChartIndices.IsValidIndex(PosVoxel) ? SolveContext->VoxelChartIndices[PosVoxel] : INDEX_NONE;
					if (PosChart != ClassChart)
					{
						return Class.Value;
					}
				}
				return PosVoxel;
			};

			// Pull in relevant attributes by its face
			OutMesh.Faces.Add(Eigen::Vector3i(
				PullVertex(Remap, Face.x(), VertexVoxel(P0)),
				PullVertex(Remap, Face.y(), VertexVoxel(P1)),
				PullVertex(Remap, Face.z(), VertexVoxel(P2))
			));
			
			continue;
		}

		// Find all overlapping voxels (again)
		OverlappingVoxels.Reset();
		Grid.FindOverlappingVoxels(P0, P1, P2, OverlappingVoxels);

		// TODO: We need to split on the seam voxel, not blindly on all voxels...
		for (int32 VoxelIndex : OverlappingVoxels)
		{
			VEUV::FVoxelData& Voxel = Grid.Voxels[VoxelIndex];
			ClippedVertices.Empty();
			
			FRegionRemap& Remap = GetRemap(VoxelIndex);
			
			// Clip the triangle to the voxel bounds
			VEUV::Geometry::ClipTriangleToVoxel(
				P0, P1, P2,
				Voxel.LogicalMin, Voxel.LogicalMax,
				ClippedVertices
			);

			if (ClippedVertices.Num() < 3)
			{
				continue;
			}

			TArray<int32, TInlineAllocator<16>> ClippedIndices;
			ClippedIndices.SetNumUninitialized(ClippedVertices.Num());

			// Vertex welding
			for (int32 PointIdx = 0; PointIdx < ClippedVertices.Num(); PointIdx++)
			{
				const Eigen::Vector3f& Point = ClippedVertices[PointIdx];
				
				// Evaluate barycentric of the clipped point
				Eigen::Vector3f Bary = VEUV::Geometry::GetBarycentrics(Point, P0, P1, P2);

				// Check if the vertex is at identity
				int32 WeldedIndex = INDEX_NONE;
				for (int32 BaryIndex = 0; BaryIndex < 3; BaryIndex++)
				{
					if (Bary[BaryIndex] <= 1.0f - 1e-4f)
					{
						continue;
					}
					
					int32 SourceIdx = Face[BaryIndex];
					
					// If there's already a remap, reuse it
					if (Remap.VertexRemap[SourceIdx] != INDEX_NONE)
					{
						WeldedIndex = Remap.VertexRemap[SourceIdx];
					}
					else
					{
						// New vertex, add it
						WeldedIndex = OutMesh.Vertices.Num();
						Remap.VertexRemap[SourceIdx] = WeldedIndex;
						OutMesh.Vertices.Add(InMesh.Vertices[SourceIdx]);

						// Associate indices
						if (OutVertexVoxelIndices)
						{
							OutVertexVoxelIndices->Add(VoxelIndex);
						}

						// Add identity source
						if (OutVertexSources)
						{
							VEUV::FVertexSource Source;
							Source.Vertices[0] = SourceIdx;
							Source.Weights[0] = 1.0f;
							OutVertexSources->Add(Source);
						}
					}
					
					break;
				}

				// If the point was welded, just keep the index
				if (WeldedIndex != INDEX_NONE)
				{
					ClippedIndices[PointIdx] = WeldedIndex;
				}
				else
				{
					// Add vertex
					int32 NewIndex = OutMesh.Vertices.Num();
					ClippedIndices[PointIdx] = NewIndex;
					OutMesh.Vertices.Add(Point);

					// Associate indices
					if (OutVertexVoxelIndices)
					{
						OutVertexVoxelIndices->Add(VoxelIndex);
					}

					// Add weighted source
					if (OutVertexSources)
					{
						VEUV::FVertexSource& Source = OutVertexSources->AddDefaulted_GetRef();
						Source.Vertices[0] = Face.x();
						Source.Vertices[1] = Face.y();
						Source.Vertices[2] = Face.z();
						Source.Weights[0] = Bary.x();
						Source.Weights[1] = Bary.y();
						Source.Weights[2] = Bary.z();
					}
				}
			}

			// Triangulate on indices
			for (int32 i = 1; i < ClippedIndices.Num() - 1; i++)
			{
				int32 I0 = ClippedIndices[0];
				int32 I1 = ClippedIndices[i];
				int32 I2 = ClippedIndices[i + 1];

				// Skip exactly-degenerate triangles, e.g. from snapping in ClipTriangleToVoxel
				if (I0 == I1 || I1 == I2 || I0 == I2)
				{
					continue;
				}

				// Register faces with voxel (for voxel split mode)
				if (bIsVoxelSplit)
				{
					Voxel.Faces.Add(OutMesh.Faces.Num());
				}
				
				OutMesh.Faces.Add(Eigen::Vector3i(I0, I1, I2));
			}
		}
	}
}

void VEUV::FSplitMesh::SplitByVoxels(
	FVoxelGrid& Grid,
	const FEigenMesh& InMesh,
	FEigenMesh& OutMesh,
	TArray<int32>& OutVertexVoxelIndices,
	TArray<FVertexSource>& OutVertexSources)
{
	SplitFacesInternal(
		Grid, 
		InMesh, OutMesh,
		true, nullptr,
		&OutVertexVoxelIndices, &OutVertexSources
	);
}

void VEUV::FSplitMesh::SplitByChartSeams(
	FVoxelGrid& Grid,
	const FGlobalSolveContext& SolveContext,
	const FEigenMesh& InMesh,
	FEigenMesh& OutMesh,
	TArray<int32>& OutVertexVoxelIndices,
	TArray<FVertexSource>& OutVertexSources)
{
	SplitFacesInternal(
		Grid, 
		InMesh, OutMesh,
		false, &SolveContext,
		&OutVertexVoxelIndices, &OutVertexSources
	);
}
