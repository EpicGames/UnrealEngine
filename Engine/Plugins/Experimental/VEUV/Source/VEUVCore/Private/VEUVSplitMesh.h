// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVVoxelGrid.h"
#include "VEUVEigen.h"
#include "VEUVGeometry.h"
#include "VEUV/VEUVTypes.h"

namespace VEUV
{
	struct FGlobalSolveContext;

	class FSplitMesh
	{
	public:
		/**
		 * Split a mesh by all voxels
		 * @param Grid Parent grid
		 * @param InMesh Mesh to split
		 * @param OutMesh Split mesh
		 * @param OutVertexVoxelIndices Optiona, voxel association
		 * @param OutVertexSources Optional, sources for interpolation
		 */
		static void SplitByVoxels(
			FVoxelGrid& Grid,
			const FEigenMesh& InMesh,
			FEigenMesh& OutMesh,
			TArray<int32>& OutVertexVoxelIndices,
			TArray<FVertexSource>& OutVertexSources);

		/**
		 * Split a mesh by the solution seams
		 * @param Grid Parent grid
		 * @param SolveContext The solution from which the seams are derived
		 * @param InMesh Mesh to split
		 * @param OutMesh Split mesh
		 * @param OutVertexVoxelIndices Optiona, voxel association
		 * @param OutVertexSources Optional, sources for interpolation
		 */
		static void SplitByChartSeams(
			FVoxelGrid& Grid,
			const FGlobalSolveContext& SolveContext,
			const FEigenMesh& InMesh,
			FEigenMesh& OutMesh,
			TArray<int32>& OutVertexVoxelIndices,
			TArray<FVertexSource>& OutVertexSources);
	};
}
