// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUV/VEUVTypes.h"

FVEUVConfig FVEUVConfig::Default = {};

// Regenerate when algorithm or default-config changes could cause a different result for same input
const FGuid FVEUVConfig::AlgorithmVersionGuid(TEXT("d8cab7e1-df9f-439c-9094-3ca179eb05ac"));
