// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUVEigen.h"

#ifndef VEUV_VALIDATE
#	define VEUV_VALIDATE 0
#endif // VEUV_VALIDATE

#define VEUV_DO_CHECK (VEUV_VALIDATE || UE_BUILD_DEBUG)

namespace VEUV
{
	FORCEINLINE void CheckFinite([[maybe_unused]] float V)
	{
#if VEUV_DO_CHECK
		ensureMsgf(FMath::IsFinite(V), TEXT("VEUV: Non-finite scalar value: %f"), V);
#endif // VEUV_DO_CHECK
	}

	FORCEINLINE void CheckFinite([[maybe_unused]] const Eigen::Vector3f& V)
	{
#if VEUV_DO_CHECK
		ensureMsgf(FMath::IsFinite(V.x()), TEXT("VEUV: Non-finite Vector3f.x: %f"), V.x());
		ensureMsgf(FMath::IsFinite(V.y()), TEXT("VEUV: Non-finite Vector3f.y: %f"), V.y());
		ensureMsgf(FMath::IsFinite(V.z()), TEXT("VEUV: Non-finite Vector3f.z: %f"), V.z());
#endif // VEUV_DO_CHECK
	}

	FORCEINLINE void CheckNormalized([[maybe_unused]] const Eigen::Vector3f& V)
	{
#if VEUV_DO_CHECK
		CheckFinite(V);
		ensureMsgf(FMath::Abs(V.squaredNorm() - 1.0f) < 0.01f, TEXT("VEUV: Vector not normalized, squaredNorm: %f"), V.squaredNorm());
#endif // VEUV_DO_CHECK
	}
}
