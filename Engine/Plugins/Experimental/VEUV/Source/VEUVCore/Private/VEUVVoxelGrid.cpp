// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVVoxelGrid.h"
#include "VEUVGeometry.h"

VEUV::FNormalization VEUV::FNormalization::Compute(TArray<Eigen::Vector3f>& Vertices)
{
	Eigen::Vector3f BoundsMin = Eigen::Vector3f::Constant(FLT_MAX);
	Eigen::Vector3f BoundsMax = Eigen::Vector3f::Constant(-FLT_MAX);
	
	// Just min/max the entire vertex set
	for (const Eigen::Vector3f& V : Vertices)
	{
		BoundsMin = BoundsMin.cwiseMin(V);
		BoundsMax = BoundsMax.cwiseMax(V);
	}
	
	// Resulting extent
	Eigen::Vector3f Extent = (BoundsMax - BoundsMin) * 0.5f;
	
	// Set center and scale
	FNormalization Result;
	Result.Center = (BoundsMin + BoundsMax) * 0.5f;
	Result.Scale = std::max({ Extent.x(), Extent.y(), Extent.z() });
	if (Result.Scale < 1e-6f)
	{
		Result.Scale = 1.0f;
	}

	// Normalize all vertices
	for (Eigen::Vector3f& V : Vertices)
	{
		V = (V - Result.Center) / Result.Scale;
	}
	
	return Result;
}

Eigen::Vector3f VEUV::FNormalization::ToWorld(const Eigen::Vector3f& Normalized) const
{
	return Normalized * Scale + Center;
}

void VEUV::FNormalization::Denormalize(TArray<Eigen::Vector3f>& Vertices) const
{
	for (Eigen::Vector3f& Vertex : Vertices)
	{
		Vertex = ToWorld(Vertex);
	}
}

void VEUV::FVoxelGrid::Distribute(const TArray<Eigen::Vector3f>& Vertices, const FVEUVConfig& InConfig)
{
	Config = InConfig;
	
	LogicalMin = Eigen::Vector3f::Constant(FLT_MAX);
	LogicalMax = Eigen::Vector3f::Constant(-FLT_MAX);

	// Get the logical bounds
	for (const Eigen::Vector3f& Pos : Vertices)
	{
		LogicalMin = LogicalMin.cwiseMin(Pos);
		LogicalMax = LogicalMax.cwiseMax(Pos);
	}

	// Nudge to avoid precision issues on bounds
	LogicalMin -= Eigen::Vector3f::Constant(1e-4f);
	LogicalMax += Eigen::Vector3f::Constant(1e-4f);
	Eigen::Vector3f Extent = LogicalMax - LogicalMin;

	// Detect near-flat axes and collapse them
	// Also pad bounds on collapsed axes to ~ non-collapsed axis voxel thickness
	constexpr float CollapseRatio = 1e-2f;
	int32 MaxAxisCount = Config.VoxelCount.GetMax();
	float MaxExtent = Extent.maxCoeff();
	float MinThickness = MaxExtent / (float)FMath::Max(1, MaxAxisCount);
	
	VoxelCount = Config.VoxelCount;
	for (int32 Axis = 0; Axis < 3; Axis++)
	{
		if (Extent[Axis] < MaxExtent * CollapseRatio)
		{
			VoxelCount[Axis] = 1;

			const float ExpandBy = (MinThickness - Extent[Axis]) * .5f;
			if (ExpandBy > 0.0f)
			{
				LogicalMin[Axis] -= ExpandBy;
				LogicalMax[Axis] += ExpandBy;
			}
		}
	}
	Extent = LogicalMax - LogicalMin;

	// Allocate voxels and map
	Voxels.SetNum(VoxelCount.X * VoxelCount.Y * VoxelCount.Z);
	VoxelIndexMap.Reserve(VoxelCount.X * VoxelCount.Y * VoxelCount.Z);

	// Note: Intentionally matching the voxel-assigning logic in FindOverlappingVoxels/GetVoxelIndexAtPosition
	const Eigen::Vector3f VoxelCountF(VoxelCount.X, VoxelCount.Y, VoxelCount.Z);
	const Eigen::Vector3f VoxelExtent = Extent.cwiseQuotient(VoxelCountF);

	// Distribute the voxels uniformly
	for (int32 z = 0; z < VoxelCount.Z; z++)
	{
		for (int32 y = 0; y < VoxelCount.Y; y++)
		{
			for (int32 x = 0; x < VoxelCount.X; x++)
			{
				// TODO: You have the same indexing schema scattered around VEUV, merge it
				int32 VoxelIndex = z * VoxelCount.Y * VoxelCount.X + y * VoxelCount.X + x;
				FVoxelData& Voxel = Voxels[VoxelIndex];

				// Add lookup
				Voxel.ID = Eigen::Vector3i(x, y, z);
				VoxelIndexMap.Add(FIntVector(x, y, z), VoxelIndex);

				// Assign min bounds
				Voxel.LogicalMin = LogicalMin + VoxelExtent.cwiseProduct(Eigen::Vector3f(
					static_cast<float>(x),
					static_cast<float>(y),
					static_cast<float>(z)
				));

				// Assign max bounds
				// Note: Uses identical floating point math to LogicalMin -- so that each voxel's
				// max should matches its neighbors corresponding min
				Voxel.LogicalMax = LogicalMin + VoxelExtent.cwiseProduct(Eigen::Vector3f(
					static_cast<float>(x + 1),
					static_cast<float>(y + 1),
					static_cast<float>(z + 1)
				));
			}
		}
	}

	// By default no corners have been allocated
	CornerAllocator = 0;
	
	// +1 since we need the upper trilinear part too
	CornerIndices.SetNumUninitialized((VoxelCount.X + 1) * (VoxelCount.Y + 1) * (VoxelCount.Z + 1));
	
	for (int32& Index : CornerIndices)
	{
		Index = INDEX_NONE;
	}
}

void VEUV::FVoxelGrid::FindOverlappingVoxels(const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& C, TArray<int32, TInlineAllocator<32>>& OutVoxelIndices) const
{
	// TODO: Invariants, hoist them
	Eigen::Vector3f Extent = LogicalMax - LogicalMin;
	Eigen::Vector3f VoxelCountF(VoxelCount.X, VoxelCount.Y, VoxelCount.Z);
	Eigen::Vector3f VoxelExtent = Extent.cwiseQuotient(VoxelCountF);

	const float TriBoxEpsilon = VoxelExtent.minCoeff() * 1e-5f;

	// Face bounds
	Eigen::Vector3f FaceMin = A.cwiseMin(B).cwiseMin(C);
	Eigen::Vector3f FaceMax = A.cwiseMax(B).cwiseMax(C);

	// Voxel min/max
	Eigen::Vector3i Min = (FaceMin - LogicalMin).cwiseQuotient(VoxelExtent).cast<int>().cwiseMax(Eigen::Vector3i::Zero()).cwiseMin(VoxelCountF.cast<int>() - Eigen::Vector3i::Ones());
	Eigen::Vector3i Max = (FaceMax - LogicalMin).cwiseQuotient(VoxelExtent).cast<int>().cwiseMax(Eigen::Vector3i::Zero()).cwiseMin(VoxelCountF.cast<int>() - Eigen::Vector3i::Ones());

	// If the same, just a single voxel, no expensive test needed
	if (Min == Max)
	{
		OutVoxelIndices.Add(Min.z() * VoxelCount.Y * VoxelCount.X + Min.y() * VoxelCount.X + Min.x());
		return;
	}

	// Check all possible voxels
	for (int32 z = Min.z(); z <= Max.z(); z++)
	{
		for (int32 y = Min.y(); y <= Max.y(); y++)
		{
			for (int32 x = Min.x(); x <= Max.x(); x++)
			{
				int32 VoxelIdx = z * VoxelCount.Y * VoxelCount.X + y * VoxelCount.X + x;
				const FVoxelData& Voxel = Voxels[VoxelIdx];

				// Intersect it
				if (Geometry::TriBoxTest(Voxel.LogicalMin, Voxel.LogicalMax, A, B, C, TriBoxEpsilon))
				{
					OutVoxelIndices.Add(VoxelIdx);
				}
			}
		}
	}
}

int32 VEUV::FVoxelGrid::AllocateCornerIndex(int32 X, int32 Y, int32 Z)
{
	int32 Idx = Z * (VoxelCount.Y + 1) * (VoxelCount.X + 1) + Y * (VoxelCount.X + 1) + X;
	
	// Reuse if already allocated
	if (CornerIndices[Idx] != INDEX_NONE)
	{
		return CornerIndices[Idx];
	}
	
	// Otherwise allocate
	int32 Index = CornerAllocator++;
	CornerIndices[Idx] = Index;
	return Index;
}

Eigen::Vector2f VEUV::FVoxelGrid::SampleUVVoxelLogical(const Eigen::VectorXf& SolvedUV, int32 VoxelIndex, const Eigen::Vector3f& LogicalPosition) const
{
	const FVoxelData& Voxel = Voxels[VoxelIndex];

	float W[8];
	Geometry::GetTrilinearWeights(LogicalPosition.x(), LogicalPosition.y(), LogicalPosition.z(), W);

	Eigen::Vector2f UV = Eigen::Vector2f::Zero();
	
	// Accumulate UV
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				int32 CornerIdx = GetCornerIndex(Voxel.ID.x() + i, Voxel.ID.y() + j, Voxel.ID.z() + k);
				if (CornerIdx != INDEX_NONE)
				{
					UV += Eigen::Vector2f(SolvedUV[CornerIdx * 2], SolvedUV[CornerIdx * 2 + 1]) * W[i * 4 + j * 2 + k];
				}
			}
		}
	}
	
	return UV;
}

Eigen::Vector2f VEUV::FVoxelGrid::SampleUV(const Eigen::VectorXf& SolvedUV, int32 VoxelIndex, const Eigen::Vector3f& LocalPosition) const
{
	const FVoxelData& Voxel = Voxels[VoxelIndex];
	Eigen::Vector3f NormPos = (LocalPosition - Voxel.LogicalMin).cwiseQuotient(Voxel.LogicalMax - Voxel.LogicalMin);
	return SampleUVVoxelLogical(SolvedUV, VoxelIndex, NormPos);
}

Eigen::Vector2f VEUV::FVoxelGrid::SampleUV(const Eigen::VectorXf& SolvedUV, const Eigen::Vector3f& Position) const
{
	return SampleUV(SolvedUV, GetVoxelIndexAtPosition(Position), Position);
}

int32 VEUV::FVoxelGrid::GetVoxelIndexAtPosition(const Eigen::Vector3f& Position) const
{
	Eigen::Vector3f Extent = LogicalMax - LogicalMin;
	Eigen::Vector3f VoxelCountF(VoxelCount.X, VoxelCount.Y, VoxelCount.Z);
	Eigen::Vector3f VoxelExtent = Extent.cwiseQuotient(VoxelCountF);

	// Get the floored normalized coordinate
	Eigen::Vector3i NormCoordinate = (Position - LogicalMin)
		.cwiseQuotient(VoxelExtent)
		.cast<int>()
		.cwiseMax(Eigen::Vector3i::Zero())
		.cwiseMin(VoxelCountF.cast<int>() - Eigen::Vector3i::Ones());

	return NormCoordinate.z() * VoxelCount.Y * VoxelCount.X + NormCoordinate.y() * VoxelCount.X + NormCoordinate.x();
}
