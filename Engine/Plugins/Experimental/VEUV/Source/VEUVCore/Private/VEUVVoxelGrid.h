// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUV/VEUVTypes.h"
#include "VEUVEigen.h"

namespace VEUV
{
	struct FVoxelEdge
	{
		Eigen::Vector3i A;
		Eigen::Vector3i B;
	};
	
	static inline FVoxelEdge GVoxelLocalPositiveEdges[] =
	{
		{ { 0, 0, 0 }, { 1, 0, 0 } },
		{ { 0, 0, 0 }, { 0, 1, 0 } },
		{ { 0, 0, 0 }, { 0, 0, 1 } },
		{ { 0, 1, 0 }, { 1, 1, 0 } },
		{ { 0, 1, 0 }, { 0, 1, 1 } },
		{ { 1, 0, 0 }, { 1, 1, 0 } },
		{ { 1, 0, 0 }, { 1, 0, 1 } },
		{ { 0, 0, 1 }, { 0, 1, 1 } },
		{ { 0, 0, 1 }, { 1, 0, 1 } },
		{ { 1, 1, 0 }, { 1, 1, 1 } },
		{ { 0, 1, 1 }, { 1, 1, 1 } },
		{ { 1, 0, 1 }, { 1, 1, 1 } },
	};
	
	struct FSampleData
	{
		Eigen::Vector3f Position;
		Eigen::Vector3f Normal;
		EVEUVSampleType Type = EVEUVSampleType::None;
	};

	struct FVoxelData
	{
		/** Local coordinate */
		Eigen::Vector3i ID = Eigen::Vector3i::Zero();
		
		/** Logical bounds */
		Eigen::Vector3f LogicalMin = Eigen::Vector3f::Zero();
		Eigen::Vector3f LogicalMax = Eigen::Vector3f::Zero();
		
		/** Average normal within this voxel */
		Eigen::Vector3f SampleAverageNormal = Eigen::Vector3f::Zero();

		/** All faces intersecting this voxel */
		TArray<int32> Faces;

		/** All samples allocated to this voxel */
		TArray<FSampleData> Samples;
		
		/** Sample probabilities */
		TArray<float> FaceAreaWeights;
		TArray<float> FaceComplexityWeights;
		
		/** Solved UVs */
		Eigen::Vector2f CornerUVs[8];
		
		/** Has this voxel been visited? */
		bool bVisited = false;
	};

	struct FNormalization
	{
		/** Origin */
		Eigen::Vector3f Center = Eigen::Vector3f::Zero();
		
		/** Uniform scale */
		float Scale = 1.0f;

		/** Compute for a set of vertices */
		static FNormalization Compute(TArray<Eigen::Vector3f>& Vertices);
		
		/** Transform a normalized position to world space */
		Eigen::Vector3f ToWorld(const Eigen::Vector3f& Normalized) const;
		
		/** Denormalize against a set of vertices */
		void Denormalize(TArray<Eigen::Vector3f>& Vertices) const;
	};

	class FVoxelGrid
	{
	public:
		/** Build config */
		FVEUVConfig Config;

		/** Active voxel count per axis. Equal to Config.VoxelCount except along near-flat axes */
		FIntVector VoxelCount = FIntVector::ZeroValue;

		/** All allocated voxels */
		TArray<FVoxelData> Voxels;
		
		/** Voxel coordinate to index */
		TMap<FIntVector, int32> VoxelIndexMap;
		
		/** Sparse corner index set */
		TArray<int32> CornerIndices;
		int32 CornerAllocator = 0;
		
		/** Logical bounds */
		Eigen::Vector3f LogicalMin = Eigen::Vector3f::Zero();
		Eigen::Vector3f LogicalMax = Eigen::Vector3f::Zero();
		
		/** Seeded voxel index */
		int32 SeedVoxelIndex = 0;
		
		/**
		 * Distribute all voxels
		 * @param Vertices Vertices to fit voxel bounds against
		 * @param Config Build config
		 */
		void Distribute(const TArray<Eigen::Vector3f>& Vertices, const FVEUVConfig& Config);

		/**
		 * Find all voxels that overlap a triangle
		 * @param OutVoxelIndices Intersecting or contained voxel(s)
		 */
		void FindOverlappingVoxels(const Eigen::Vector3f& A, const Eigen::Vector3f& B, const Eigen::Vector3f& C, TArray<int32, TInlineAllocator<32>>& OutVoxelIndices) const;
		
		/**
		 * Sample the UVs of a particular solution
		 * @param SolvedUV Solved region
		 * @param VoxelIndex Voxel index to sample against
		 * @param LogicalPosition Local position within the logical space, not whole container
		 * @return Evaluated UV
		 */
		Eigen::Vector2f SampleUVVoxelLogical(const Eigen::VectorXf& SolvedUV, int32 VoxelIndex, const Eigen::Vector3f& LogicalPosition) const;

		/**
		 * Sample the UVs of a particular solution
		 * @param SolvedUV Solved region
		 * @param VoxelIndex Voxel index to sample against
		 * @param Position The grid wise position
		 * @return Evaluated UV
		 */
		Eigen::Vector2f SampleUV(const Eigen::VectorXf& SolvedUV, int32 VoxelIndex, const Eigen::Vector3f& Position) const;

		/**
		 * Sample the UVs of a particular solution
		 * @param SolvedUV Solved region
		 * @param Position The grid wise position
		 * @return Evaluated UV
		 */
		Eigen::Vector2f SampleUV(const Eigen::VectorXf& SolvedUV, const Eigen::Vector3f& Position) const;

		/**
		 * Find the voxel index that contains a position via floor lookup.
		 * @param Position The grid wise position
		 * @return Voxel index of Position clamped to grid bounds
		 */
		int32 GetVoxelIndexAtPosition(const Eigen::Vector3f& Position) const;

		/**
		 * Allocate a new corner index
		 * @return Index, may be reused
		 */
		int32 AllocateCornerIndex(int32 X, int32 Y, int32 Z);

		/**
		 * Get a potentially allocated corner index
		 * @return INDEX_NONE if not allocated
		 */
		int32 GetCornerIndex(int32 X, int32 Y, int32 Z) const
		{
			int32 Idx = Z * (VoxelCount.Y + 1) * (VoxelCount.X + 1) + Y * (VoxelCount.X + 1) + X;
			return CornerIndices[Idx];
		}
	};
}
