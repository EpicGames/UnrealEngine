// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUV/VEUVTypes.h"

#define UE_API VEUVCORE_API

namespace VEUV
{
	struct FDebugCapture
	{		
		/** Debug sample point in world space */
		struct FDebugSample
		{
			FVector Position;
			FColor Color;
			EVEUVSampleType Type = EVEUVSampleType::None;
		};

		/** Per-chart EV-orientation debug data */
		struct FChartEVDebug
		{
			FVector2f Mean;
			FVector2f EV0;
			FVector2f EV1;
		};
		
		/** World voxel data */
		struct FVoxelDebug
		{
			FVector Center;
			FVector Extent;
			bool bOccupied = false;
		};
		
		/** Solver geometry snapshot */
		struct FGeometrySnapshot
		{
			FString Name;
			TArray<FInt32Vector3> Faces;
			TArray<FVector2f> VertexUVs;
		};
		
		/** Optional, name of this capture */
		FString SectionName;
		
		/** Time built */
		FDateTime Timestamp;
		
		/** Config used during builds */
		FVEUVConfig Config;
		
		/** If true, user requested snapshot capturing */
		bool bWithSnapshots = false;
		
		/** Error history */
		TArray<float> R78ErrorHistory;
		TArray<float> R9ErrorHistory;
		TArray<float> R9GradNormHistory;
		TArray<float> R10GradNormHistory;
		
		/** Snapshot history */
		TArray<FGeometrySnapshot> Snapshots;

		/** The full VEUV result */
		FResult Result;

		TArray<FChartEVDebug> ChartEVs;
		TArray<FDebugSample> Samples;
		TArray<FVoxelDebug> Voxels;
	};

	class FDebugHistory
	{
	public:
		static UE_API FDebugHistory& Get();

		void Add(TSharedPtr<FDebugCapture> Capture)
		{
			{
				FScopeLock ScopeLock(&CriticalSection);
				Captures.Add(Capture);
				while (Captures.Num() > MaxHistory)
				{
					Captures.RemoveAt(0);
				}
			}
			OnCaptureAdded.Broadcast();
		}

		TArray<TSharedPtr<FDebugCapture>> GetCaptures() const
		{
			FScopeLock ScopeLock(&CriticalSection);
			return Captures;
		}

		void Clear()
		{
			FScopeLock ScopeLock(&CriticalSection);
			Captures.Reset();
		}

		int32 MaxHistory = 32;

		DECLARE_MULTICAST_DELEGATE(FOnCaptureAdded);
		FOnCaptureAdded OnCaptureAdded;

	private:
		FDebugHistory() = default;

		mutable FCriticalSection CriticalSection;
		TArray<TSharedPtr<FDebugCapture>> Captures;
	};
}

#undef UE_API
