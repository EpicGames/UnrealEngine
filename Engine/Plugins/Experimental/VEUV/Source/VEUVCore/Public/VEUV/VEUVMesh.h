// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace VEUV
{
	struct FMesh
	{
		/** All vertices, assumed all used */
		TArray<FVector3f> Vertices;
		
		/** All triangles */
		TArray<FInt32Vector3> Faces;
	};
}
