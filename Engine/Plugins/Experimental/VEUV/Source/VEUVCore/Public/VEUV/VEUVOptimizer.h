// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUV/VEUVMesh.h"
#include "VEUV/VEUVTypes.h"

#define UE_API VEUVCORE_API

namespace VEUV
{
	class FOptimizer
	{
	public:
		/**
		 * Create and optimize a UV parameterization for a given mesh
		 * @param InputMesh Mesh to parameterize for
		 * @param Config Optimization config
		 * @return Resulting mesh
		 */
		static UE_API FResult Compute(const FMesh& InputMesh, const FVEUVConfig& Config);
	};
}

#undef UE_API
