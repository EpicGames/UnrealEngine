// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VEUV/VEUVMesh.h"
#include "VEUVTypes.generated.h"

/** Sample type classification */
enum class EVEUVSampleType : uint8
{
	None,
	Area,
	Complexity,
	Adaptive
};

UENUM()
enum class EVEUVVisualizationDebugMode : uint8
{
	/** Write the actual UVs */
	None,

	/** Debug, write the normalized probabilities */
	P_Area,
	P_Complexity,
	P_Combined
};

USTRUCT()
struct FVEUVSamplingConfig
{
	GENERATED_BODY()

	/** Total sample budget, distributed across budgets */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=1))
	int32 TotalSamples = 4096;

	/** Complexity amplification, 0 -> pure area sampling */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=0.0))
	float ComplexityAlpha = 5.0f;

	/** Fraction of samples allocated to complex areas compared to area sampling */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=0.0, ClampMax=1.0))
	float ComplexityBlend = 0.5f;

	/** Number of normal bins for complexity samples */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=1, ClampMax=128))
	int32 ComplexityNormalBins = 12;

	/** Absolute size of each complexity field cell */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=1.0))
	float ComplexityFieldCellSize = 256.0f;

	/** Hard limit to number of complexity fields per axis */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=2, ClampMax=256))
	int32 ComplexityFieldMaxResolution = 64;

	/** Number of dilution iterations for the complexity field */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=0, ClampMax=32))
	int32 ComplexityDilutionIterations = 2;

	/** Number of sub-cells for adaptive sampling */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=0, ClampMax=16))
	int32 AdaptiveSubdivisions = 4;

	/** Fraction of total samples to allocate for adaptive samples */
	UPROPERTY(EditAnywhere, Category="Sampling", meta=(ClampMin=0.0, ClampMax=1.0))
	float AdaptiveFraction = 0.5f;
};

USTRUCT()
struct FVEUVSolverConfig
{
	GENERATED_BODY()

	/** Enable standard Hessian discrete Laplacian regularization */
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bEnableDiscreteLaplace = true;

	/** Enable Hessian nudge regularization */
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bEnableDiagonalLMDamping = true;

	/** Enable the R9 global area-preservation solve. */
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bEnableGlobalSolve = true;

	/** Hessian regularization alpha */
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=0.0, ClampMax=1.0))
	float DiagonalNudgeAlpha = 1e-6f;

	/** Number of R9 global solve iterations **/
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=1, ClampMax=10000))
	int32 GlobalIterations = 1000;

	/** Learning rate for the global R9 solve */
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=0.0, ClampMax=1.0))
	float GlobalSolveLR = 1e-2f;

	/**
	 * Regularization alpha between R78 and R9ab.
	 *    R78 + R9ab * K * Alpha
	 *
	 * Higher values allocates more importance to area preservation, but sacrifices orthogonality and conformity.
	 */
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=0.0))
	float GlobalSolveR9abRegAlpha = 1.0f;

	/**
	 * Regularization alpha between R78 and R10.
	 *    R78 + R10 * K * Alpha
	 *
	 * Higher values allocates more importance to map injectivity, but sacrifices orthogonality and conformity,
	 * and area preservation (indirectly).
	 */
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=0.0))
	float GlobalSolveR10RegAlpha = 1.0f;

	/**
	 * If true, uses implicit gradients with back-propagation instead of solving through the Hessian form.
	 * Much faster, but loses second order information (* not used yet).
	 **/
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bGlobalSolveGDFirstOrder = true;

	/**
	 * If true, enables a general injectivity conforming pre-pass that better pre-conditions the R9ab solve pass.
	 **/
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bEnableInjectiveTerm = false;

	/**
	 * If true, enables a linear (Cholesky) solve of the non-linear system as if it was linear to act as a pre-conditioning,
	 * reduces the amount of iterations required.
	 **/
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bGlobalSolveEnableLinearPrecond = true;

	/**
	 * Enable per-chart R9 refinement after charting.
	 * Individual charts will have a much easier time optimizing in isolation.
	 **/
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bEnableChartRefinement = true;

	/**
	 * If true, forces injectivity of each separate chart through a R10 hard constraint
	 **/
	UPROPERTY(EditAnywhere, Category="Solver")
	bool bChartRefinementForceInjectivity = false;

	/** Fraction of GlobalIterations used for per-chart refinement. */
	UPROPERTY(EditAnywhere, Category="Solver", meta=(ClampMin=0.0, ClampMax=1.0))
	float ChartRefinementIterationFraction = 0.2f;
};

USTRUCT()
struct FVEUVChartingConfig
{
	GENERATED_BODY()

	/** Enable rasterization-based overlap detection */
	UPROPERTY(EditAnywhere, Category="Charting")
	bool bEnableRasterization = true;

	/** Rasterization width for overlap detection */
	UPROPERTY(EditAnywhere, Category="Charting", meta=(ClampMin=16, ClampMax=4096))
	int32 RasterizationWidth = 256;

	/** Enable overlap detection, if false, single charting */
	UPROPERTY(EditAnywhere, Category="Charting")
	bool bEnableOverlapChecks = true;

	/** Epsilon for chart UV-rect checks */
	UPROPERTY(EditAnywhere, Category="Charting", meta=(ClampMin=0.0))
	float UVRectOverlapEpsilon = 0.0f;
};

USTRUCT()
struct FVEUVPackingConfig
{
	GENERATED_BODY()

	/** Enable PCA-based chart reorientation for tighter packing */
	UPROPERTY(EditAnywhere, Category="Packing")
	bool bEnableChartReorientation = true;

	/** Enable atlas packing, if false, charts have no spatial relation */
	UPROPERTY(EditAnywhere, Category="Packing")
	bool bEnableAtlasPacking = true;

	/** Enable per-chart normalization, only relevant when packing is disabled */
	UPROPERTY(EditAnywhere, Category="Packing")
	bool bEnableUVNormalization = false;

	/** Atlas packing width in texels */
	UPROPERTY(EditAnywhere, Category="Packing", meta=(ClampMin=16, ClampMax=8192))
	int32 AtlasPackWidth = 512;
};

USTRUCT()
struct FVEUVConfig
{
	GENERATED_BODY()

	static VEUVCORE_API FVEUVConfig Default;

	/**
	 * Identity of the VEUV algorithm and its built-in defaults --
	 * Regenerate when algorithm or default-config changes could cause a different result for same input
	 */
	static VEUVCORE_API const FGuid AlgorithmVersionGuid;

	/** Helper for a low-quality/fast-iteration build */
	static FVEUVConfig LowQuality()
	{
		FVEUVConfig Out = Default;
		Out.Label = TEXT("LowQuality");
		Out.Solver.GlobalIterations = 1;
		Out.Solver.bEnableChartRefinement = false;
		Out.Sampling.AdaptiveFraction = 0;
		return Out;
	}

	/** Debugging label for this build */
	const TCHAR* Label = TEXT("Standard");

	/** Number of voxels per axis */
	UPROPERTY(EditAnywhere, Category="Grid", meta=(ClampMin=1, ClampMax=64))
	FIntVector VoxelCount = FIntVector(4, 4, 4);

	/** Optional, world transform for debug drawing. Applied to all debug points/boxes. */
	UPROPERTY(EditAnywhere, Category="Debug")
	FTransform DebugWorldTransform = FTransform::Identity;

	/** Optional, visualizes certain attributes in the UV channels */
	UPROPERTY(EditAnywhere, Category="Debug")
	EVEUVVisualizationDebugMode UVVisualizeMode = EVEUVVisualizationDebugMode::None;

	/** Normalize vertices to [-1, 1], for most geometry this is required for well-formed optimization */
	UPROPERTY(EditAnywhere, Category="Grid")
	bool bEnableVertexNormalization = true;

	/** Sampling settings */
	UPROPERTY(EditAnywhere, Category="Config")
	FVEUVSamplingConfig Sampling;

	/** Solution solver settings */
	UPROPERTY(EditAnywhere, Category="Config")
	FVEUVSolverConfig Solver;

	/** Charting settings */
	UPROPERTY(EditAnywhere, Category="Config")
	FVEUVChartingConfig Charting;

	/** Atlas packing settings */
	UPROPERTY(EditAnywhere, Category="Config")
	FVEUVPackingConfig Packing;
};

namespace VEUV
{
	enum class EChartStatus : uint8
	{
		Valid          = 0,
		Empty          = 1 << 0, // No voxels / no faces in this chart
		NonFinite      = 1 << 1, // At least one vertex UV is NaN/Inf
		PackingFailed  = 1 << 2, // Atlas packer couldn't allocate this chart at any power
		Inverted       = 1 << 3, // At least one face has UV winding opposite to chart majority
	};
	ENUM_CLASS_FLAGS(EChartStatus);

	struct FChart
	{
		/** All voxels belonging to this chart */
		TArray<int32> VoxelIndices;

		/** Axis aligned UV bounds */
		FBox2f Bounds = FBox2f(FVector2f(MAX_flt, MAX_flt), FVector2f(-MAX_flt, -MAX_flt));

		/** UV bounds after atlas allocation */
		FBox2f PackedBounds = FBox2f(FVector2f(0.0f, 0.0f), FVector2f(0.0f, 0.0f));

		/** World surface area */
		float Area = 0.0f;

		/** Per-chart status flags */
		EChartStatus Status = EChartStatus::Valid;
	};

	struct FStatistics
	{
		double DistributeMs = 0;
		double SplitFacesMs = 0;
		double SampleVoxelsMs = 0;
		double ContinuousField78Ms = 0;
		double GlobalField9Ms = 0;
		int32 GlobalField9Iterations = 0;
		double GlobalField10Ms = 0;
		int32 GlobalField10Iterations = 0;
		double NormalizeMs = 0;
		double ChartingMs = 0;
		double ChartRefinementMs = 0;
		int32 ChartRefinementIterations = 0;
		double ReorientMs = 0;
		double PackMs = 0;
		double SplitFacesChartMs = 0;
		double EvalUVsMs = 0;
		double AdaptiveResampleMs = 0;
		double TotalMs = 0;

		const TCHAR* Label = nullptr;
		int32 InputVertices = 0;
		int32 InputFaces = 0;
		int32 OutputVertices = 0;
		int32 OutputFaces = 0;
		int32 ChartCount = 0;
		int32 VoxelCount = 0;
		int32 OccupiedVoxels = 0;
		int32 TotalSamples = 0;
		int32 AdaptiveSamplesAdded = 0;
		FIntVector ComplexityFieldResolution = FIntVector::ZeroValue;
		int32 RasterizationWidth = 0;
		int32 InvertedFaceCount = 0;
		int32 UnassignedFaceCount = 0;
		int32 FailedChartCount = 0;
		float InvertedUVArea = 0.0f; // Sum of |signed UV area| over inverted faces
		float TotalUVArea    = 0.0f; // Sum of |signed UV area| over all faces

		FString ToString() const
		{
			FString S;
			S += FString::Printf(TEXT("[VEUV] Build Report - %s\n"), Label);
			S += FString::Printf(TEXT("  Mesh\n"));
			S += FString::Printf(TEXT("    Input          %d verts, %d faces\n"), InputVertices, InputFaces);
			S += FString::Printf(TEXT("    Output         %d verts, %d faces\n"), OutputVertices, OutputFaces);
			S += FString::Printf(TEXT("  Grid\n"));
			S += FString::Printf(TEXT("    Voxels         %d total, %d occupied\n"), VoxelCount, OccupiedVoxels);
			S += FString::Printf(TEXT("    Charts         %d (%d failed)\n"), ChartCount, FailedChartCount);
			const double InvertedAreaPct = TotalUVArea > 0.0f ? 100.0 * InvertedUVArea / TotalUVArea : 0.0;
			S += FString::Printf(TEXT("    Inverted Faces %d (%.3f%% of UV area)\n"), InvertedFaceCount, InvertedAreaPct);
			const double UnassignedPct = OutputFaces > 0 ? 100.0 * UnassignedFaceCount / OutputFaces : 0.0;
			S += FString::Printf(TEXT("    Unassigned     %d (%.3f%% of faces)\n"), UnassignedFaceCount, UnassignedPct);
			S += FString::Printf(TEXT("    Samples        %d (%.1f/voxel)\n"), TotalSamples, OccupiedVoxels > 0 ? static_cast<double>(TotalSamples) / OccupiedVoxels : 0.0);
			S += FString::Printf(TEXT("    Adaptive       +%d samples\n"), AdaptiveSamplesAdded);
			S += FString::Printf(TEXT("    Field          %dx%dx%d\n"), ComplexityFieldResolution.X, ComplexityFieldResolution.Y, ComplexityFieldResolution.Z);
			S += FString::Printf(TEXT("    Rast Width     %d\n"), RasterizationWidth);
			S += FString::Printf(TEXT("  Timing\n"));
			S += FString::Printf(TEXT("    Distribute     %7.2f ms\n"), DistributeMs);
			S += FString::Printf(TEXT("    Split Faces    %7.2f ms\n"), SplitFacesMs);
			S += FString::Printf(TEXT("    Sample Voxels  %7.2f ms\n"), SampleVoxelsMs);
			S += FString::Printf(TEXT("    Solve R78      %7.2f ms\n"), ContinuousField78Ms);
			S += FString::Printf(TEXT("    Adaptive       %7.2f ms\n"), AdaptiveResampleMs);
			S += FString::Printf(TEXT("    Solve R10      %7.2f ms (%d iters)\n"), GlobalField10Ms, GlobalField10Iterations);
			S += FString::Printf(TEXT("    Solve R9       %7.2f ms (%d iters)\n"), GlobalField9Ms, GlobalField9Iterations);
			S += FString::Printf(TEXT("    Normalize      %7.2f ms\n"), NormalizeMs);
			S += FString::Printf(TEXT("    Charting       %7.2f ms\n"), ChartingMs);
			S += FString::Printf(TEXT("    Chart Refine   %7.2f ms (%d iters)\n"), ChartRefinementMs, ChartRefinementIterations);
			S += FString::Printf(TEXT("    Reorient       %7.2f ms\n"), ReorientMs);
			S += FString::Printf(TEXT("    Pack           %7.2f ms\n"), PackMs);
			S += FString::Printf(TEXT("    Split Seams    %7.2f ms\n"), SplitFacesChartMs);
			S += FString::Printf(TEXT("    Eval UVs       %7.2f ms\n"), EvalUVsMs);
			S += FString::Printf(TEXT("    --------------------------------\n"));
			S += FString::Printf(TEXT("    Total          %7.2f ms\n"), TotalMs);
			return S;
		}

		void Log() const
		{
			UE_LOG(LogTemp, Display, TEXT("%s"), *ToString());
		}
	};

	struct FVertexSource
	{
		int32 Vertices[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
		float Weights[3] = { 1.0f, 0.0f, 0.0f };
	};
	
	struct FResult
	{
		/** Resulting mesh, may have splits */
		FMesh OutputMesh;

		/** Produced vertex UVs, same indexing as output mesh. */
		TArray<FVector2f> VertexUVs;

		/** All produced charts */
		TArray<FChart> Charts;

		/** Per-face chart index (-1 if unassigned) */
		TArray<int32> FaceChartIndices;

		/** Interpolation sources for additional attributes, for reconstructing additional attributes and interpolating on splits */
		TArray<FVertexSource> VertexSources;

		/** Timing statistics */
		FStatistics Stats;

		/** Bitwise-OR of per-chart statuses. Equals EChartStatus::Valid (0) iff every chart is Valid. Collection-level concerns (e.g. unassigned faces) are reported via FStatistics. */
		EChartStatus Status = EChartStatus::Valid;
	};
}
