// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VEUVCore : ModuleRules
{
	public VEUVCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Eigen",
		});
	}
}
