// Copyright Epic Games, Inc. All Rights Reserved.

#include "VEUVDebugPanel.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "DrawDebugHelpers.h"
#include "PropertyEditorModule.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Rendering/DrawElementTypes.h"
#include "Styling/CoreStyle.h"
#include "Engine/Engine.h"
#include "Editor.h"
#include "LevelEditorViewport.h"

namespace
{
	float UVPhase(float X)
	{
		float F = FMath::Frac(X * 5.0f);
		float D = FMath::Min(F, 1.0f - F);
		return 1.0f - FMath::SmoothStep(0.0f, 0.03f, D);
	}

	FColor UVToColor(FVector2f UV)
	{
		float Phase = FMath::Max(UVPhase(UV.X * 20), UVPhase(UV.Y * 20));
		FLinearColor Base(UV.X, UV.Y, 0.0f, 1.0f);
		FLinearColor Grid(0.75f, 0.75f, 0.75f, 1.0f);
		return FMath::Lerp(Base, Grid, Phase).ToFColor(true);
	}

	uint32 MurmurMix(uint32 Hash)
	{
		Hash ^= Hash >> 16;
		Hash *= 0x85ebca6b;
		Hash ^= Hash >> 13;
		Hash *= 0xc2b2ae35;
		Hash ^= Hash >> 16;
		return Hash;
	}

	FColor ChartToColor(int32 Index)
	{
		if (Index == INDEX_NONE)
		{
			return FColor::Black;
		}
		uint32 Hash = MurmurMix(static_cast<uint32>(Index) + 1);
		return FColor(
			static_cast<uint8>((Hash >> 0) & 0xFF),
			static_cast<uint8>((Hash >> 8) & 0xFF),
			static_cast<uint8>((Hash >> 16) & 0xFF),
			255);
	}
}

void SVEUVCanvas::Construct(const FArguments&)
{
	SetClipping(EWidgetClipping::ClipToBounds);
}

void SVEUVCanvas::Clear()
{
	ClearContent();
	bUniformZoom = true;
	bFlipY = false;
	ViewOffset = FVector2f(0.0f, 0.0f);
	ViewZoom = FVector2f(1.0f, 1.0f);
}

FVector2f SVEUVCanvas::GetScale(const FGeometry& Geometry) const
{
	if (bUniformZoom)
	{
		float S = FMath::Max(1.0f, FMath::Min(static_cast<float>(Geometry.GetLocalSize().X), static_cast<float>(Geometry.GetLocalSize().Y)));
		return FVector2f(S, S);
	}
	return FVector2f(
		FMath::Max(1.0f, static_cast<float>(Geometry.GetLocalSize().X)),
		FMath::Max(1.0f, static_cast<float>(Geometry.GetLocalSize().Y)));
}

FVector2f SVEUVCanvas::ScreenToNormalized(const FGeometry& Geometry, FVector2f ScreenPos) const
{
	FVector2f Local = FVector2f(Geometry.AbsoluteToLocal(FVector2D(ScreenPos.X, ScreenPos.Y)));
	return Local / GetScale(Geometry);
}

FVector2f SVEUVCanvas::ScreenToDataUV(const FGeometry& Geometry, FVector2f ScreenPos) const
{
	FVector2f Norm = ScreenToNormalized(Geometry, ScreenPos);
	FVector2f UV = (Norm - ViewOffset) / ViewZoom;
	if (bFlipY)
	{
		UV.Y = -UV.Y;
	}
	return UV;
}

void SVEUVCanvas::FitToContent()
{
	ViewOffset = FVector2f(0.0f, 0.0f);
	ViewZoom = FVector2f(1.0f, 1.0f);

	if (!HasContent())
	{
		return;
	}

	FBox2f Bounds = GetContentBounds();
	FVector2f Extent = Bounds.Max - Bounds.Min;
	if (Extent.X < FLT_EPSILON || Extent.Y < FLT_EPSILON)
	{
		return;
	}

	FVector2f LocalSize(GetCachedGeometry().GetLocalSize().X, GetCachedGeometry().GetLocalSize().Y);
	FVector2f Scale = GetScale(GetCachedGeometry());
	float NormWidth = LocalSize.X / Scale.X;
	float NormHeight = LocalSize.Y / Scale.Y;

	constexpr float Margin = 0.05f;
	float AvailableWidth = NormWidth - Margin * 2.0f;
	float AvailableHeight = NormHeight - Margin * 2.0f;

	if (bUniformZoom)
	{
		float Zoom = FMath::Max(1e-6f, FMath::Min(AvailableWidth / Extent.X, AvailableHeight / Extent.Y));
		ViewZoom = FVector2f(Zoom, Zoom);
	}
	else
	{
		ViewZoom.X = FMath::Max(1e-6f, AvailableWidth / Extent.X);
		ViewZoom.Y = FMath::Max(1e-6f, AvailableHeight / Extent.Y);
	}

	ViewOffset.X = Margin - Bounds.Min.X * ViewZoom.X;
	if (bFlipY)
	{
		ViewOffset.Y = Margin + Bounds.Max.Y * ViewZoom.Y;
	}
	else
	{
		ViewOffset.Y = Margin - Bounds.Min.Y * ViewZoom.Y;
	}

	OnViewChanged.ExecuteIfBound();
}

int32 SVEUVCanvas::DrawGrid(
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	FVector2f Scale,
	FVector2f Origin,
	FVector2f AbsOrigin,
	FVector2f ViewportSize) const
{
	auto DrawLine = [&](FVector2f A, FVector2f B, const FLinearColor& Color, float Thickness = 1.0f)
	{
		FSlateDrawElement::MakeLines(
			OutDrawElements, LayerId, FPaintGeometry(),
			{ FVector2D(A), FVector2D(B) }, 
			ESlateDrawEffect::None, Color, true, Thickness);
	};
	
	// Compute visible data-space bounds
	FVector2f ViewMin, ViewMax;
	ViewMin.X = ((AbsOrigin.X - Origin.X) / Scale.X - ViewOffset.X) / ViewZoom.X;
	ViewMax.X = ((AbsOrigin.X + ViewportSize.X - Origin.X) / Scale.X - ViewOffset.X) / ViewZoom.X;

	if (bFlipY)
	{
		ViewMin.Y = -((AbsOrigin.Y + ViewportSize.Y - Origin.Y) / Scale.Y - ViewOffset.Y) / ViewZoom.Y;
		ViewMax.Y = -((AbsOrigin.Y - Origin.Y) / Scale.Y - ViewOffset.Y) / ViewZoom.Y;
	}
	else
	{
		ViewMin.Y = ((AbsOrigin.Y - Origin.Y) / Scale.Y - ViewOffset.Y) / ViewZoom.Y;
		ViewMax.Y = ((AbsOrigin.Y + ViewportSize.Y - Origin.Y) / Scale.Y - ViewOffset.Y) / ViewZoom.Y;
	}

	float Extent = FMath::Max(ViewMax.X - ViewMin.X, ViewMax.Y - ViewMin.Y);
	if (Extent < FLT_EPSILON)
	{
		return LayerId;
	}

	float StepPacing = Extent / 16.0f;
	float GridStep = FMath::Pow(10.0f, FMath::FloorToFloat(FMath::LogX(10.0f, StepPacing)));
	
	if (StepPacing / GridStep > 5.0f)
	{
		GridStep *= 5.0f;
	}
	else if (StepPacing / GridStep > 2.0f)
	{
		GridStep *= 2.0f;
	}

	FLinearColor GridColor(0.15f, 0.15f, 0.15f, 1.0f);
	FLinearColor XAxisColor = FLinearColor::Red;
	FLinearColor YAxisColor = FLinearColor::Green;

	// Tick label formatting
	int32 TickDecimals = FMath::Max(0, FMath::CeilToInt32(-FMath::LogX(10.0f, GridStep)));
	auto FormatTick = [TickDecimals](float Value) -> FString
	{
		ANSICHAR Buf[64];
		FCStringAnsi::Snprintf(Buf, sizeof(Buf), "%.*f", TickDecimals, Value);
		return FString(Buf);
	};
	FSlateFontInfo TickFont = FCoreStyle::Get().GetFontStyle("SmallFont");
	TickFont.Size = 11;

	constexpr int32 MaxGridLines = 128;

	// Vertical grid lines + X tick labels
	{
		int32 Count = 0;
		for (float X = FMath::FloorToFloat(ViewMin.X / GridStep) * GridStep; X <= ViewMax.X && Count < MaxGridLines; X += GridStep, Count++)
		{
			DrawLine(UVToScreen(FVector2f(X, ViewMin.Y), Scale, Origin), UVToScreen(FVector2f(X, ViewMax.Y), Scale, Origin), GridColor);

			FVector2f TickPos = UVToScreen(FVector2f(X, 0), Scale, Origin);
			FString XLabel = FormatTick(X);
			float CharY = TickPos.Y + 4.0f;
			for (int32 CharIdx = 0; CharIdx < XLabel.Len(); CharIdx++)
			{
				FSlateDrawElement::MakeText(
					OutDrawElements, LayerId,
					FPaintGeometry(FVector2D(TickPos.X - 4, CharY), FVector2D(14, 14), 1.0f),
					XLabel.Mid(CharIdx, 1), TickFont, ESlateDrawEffect::None, GridColor);
				CharY += 12.0f;
			}
		}
	}

	// Horizontal grid lines + Y tick labels
	{
		int32 Count = 0;
		for (float Y = FMath::FloorToFloat(ViewMin.Y / GridStep) * GridStep; Y <= ViewMax.Y && Count < MaxGridLines; Y += GridStep, Count++)
		{
			DrawLine(UVToScreen(FVector2f(ViewMin.X, Y), Scale, Origin), UVToScreen(FVector2f(ViewMax.X, Y), Scale, Origin), GridColor);

			FVector2f TickPos = UVToScreen(FVector2f(0, Y), Scale, Origin);
			FString YLabel = FormatTick(Y);
			float LabelWidth = YLabel.Len() * 7.0f;
			FSlateDrawElement::MakeText(
				OutDrawElements, LayerId,
				FPaintGeometry(FVector2D(TickPos.X - LabelWidth - 4.0f, TickPos.Y - 7.0f), FVector2D(LabelWidth + 4.0f, 14), 1.0f),
				YLabel, TickFont, ESlateDrawEffect::None, GridColor);
		}
	}

	// UV 0-1 unit box
	if (ShouldDrawUnitBox())
	{
		FLinearColor UnitBoxColor(0.4f, 0.4f, 0.4f, 1.0f);
		FVector2f C00 = UVToScreen(FVector2f(0, 0), Scale, Origin);
		FVector2f C10 = UVToScreen(FVector2f(1, 0), Scale, Origin);
		FVector2f C11 = UVToScreen(FVector2f(1, 1), Scale, Origin);
		FVector2f C01 = UVToScreen(FVector2f(0, 1), Scale, Origin);
		DrawLine(C00, C10, UnitBoxColor, 2.0f);
		DrawLine(C10, C11, UnitBoxColor, 2.0f);
		DrawLine(C11, C01, UnitBoxColor, 2.0f);
		DrawLine(C01, C00, UnitBoxColor, 2.0f);
	}

	// Axes
	DrawLine(UVToScreen(FVector2f(ViewMin.X, 0), Scale, Origin), UVToScreen(FVector2f(ViewMax.X, 0), Scale, Origin), XAxisColor, 1.5f);
	DrawLine(UVToScreen(FVector2f(0, ViewMin.Y), Scale, Origin), UVToScreen(FVector2f(0, ViewMax.Y), Scale, Origin), YAxisColor, 1.5f);

	return LayerId + 1;
}

void SVEUVCanvas::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bPendingFitToContent && GetCachedGeometry().GetLocalSize().X > 0 && GetCachedGeometry().GetLocalSize().Y > 0)
	{
		bPendingFitToContent = false;
		FitToContent();
	}
}

int32 SVEUVCanvas::OnPaint(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (!HasContent())
	{
		return LayerId;
	}

	FVector2f AbsOrigin(AllottedGeometry.AbsolutePosition.X, AllottedGeometry.AbsolutePosition.Y);
	FVector2f AbsSize(AllottedGeometry.GetAbsoluteSize().X, AllottedGeometry.GetAbsoluteSize().Y);

	FVector2f Scale;
	if (bUniformZoom)
	{
		float S = FMath::Min(AbsSize.X, AbsSize.Y);
		Scale = FVector2f(S, S);
	}
	else
	{
		Scale = AbsSize;
	}

	if (Scale.X < 1.0f || Scale.Y < 1.0f)
	{
		return LayerId;
	}

	FVector2f Origin = AbsOrigin;

	LayerId = DrawGrid(OutDrawElements, LayerId, Scale, Origin, AbsOrigin, AbsSize);
	LayerId = PaintContent(AllottedGeometry, OutDrawElements, LayerId, Scale, Origin);

	return LayerId;
}

FVector2D SVEUVCanvas::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(FLT_MAX, FLT_MAX);
}

FReply SVEUVCanvas::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton ||
		MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsPanning = true;
		PanStart = FVector2f(MouseEvent.GetScreenSpacePosition().X, MouseEvent.GetScreenSpacePosition().Y);
		PanOffsetStart = ViewOffset;
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SVEUVCanvas::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsPanning)
	{
		bIsPanning = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SVEUVCanvas::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsPanning)
	{
		FVector2f Current(MouseEvent.GetScreenSpacePosition().X, MouseEvent.GetScreenSpacePosition().Y);
		FVector2f Scale = GetScale(MyGeometry);
		FVector2f Delta = (Current - PanStart) / (Scale * MyGeometry.Scale);
		ViewOffset = PanOffsetStart + Delta;
		OnViewChanged.ExecuteIfBound();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SVEUVCanvas::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2f NormPos = ScreenToNormalized(MyGeometry, FVector2f(MouseEvent.GetScreenSpacePosition()));
	FVector2f NormUV = (NormPos - ViewOffset) / ViewZoom;

	float ZoomDelta = MouseEvent.GetWheelDelta() > 0 ? 1.1f : 1.0f / 1.1f;
	ViewZoom.X = FMath::Clamp(ViewZoom.X * ZoomDelta, 1e-6f, 1e6f);
	ViewZoom.Y = FMath::Clamp(ViewZoom.Y * ZoomDelta, 1e-6f, 1e6f);

	ViewOffset = NormPos - NormUV * ViewZoom;

	OnViewChanged.ExecuteIfBound();
	return FReply::Handled();
}

void SVEUVMeshViewport::Construct(const FArguments& InArgs)
{
	SVEUVCanvas::Construct(SVEUVCanvas::FArguments());

	// Try to load the debug UV grid material (optional, falls back to vertex colors)
	if (UMaterialInterface* Mat = LoadObject<UMaterialInterface>(nullptr, TEXT("/VEUV/M_VeuvDebugUVGrid")))
	{
		GridMaterialBrush = MakeShared<FSlateMaterialBrush>(FVector2D(64, 64));
		GridMaterialBrush->SetMaterial(Mat);
	}
}

void SVEUVMeshViewport::BuildMesh(TSharedPtr<VEUV::FDebugCapture> InCapture, EVEUVVisualizationMode Mode)
{
	bUniformZoom = true;
	bFlipY = false;
	bUseMaterialBrush = (Mode == EVEUVVisualizationMode::UVLayout);

	Capture = InCapture;
	const VEUV::FResult& Result = Capture->Result;
	const TArray<FVector2f>& UVs = Result.VertexUVs;
	const TArray<FIntVector>& Faces = Result.OutputMesh.Faces;

	if (Mode == EVEUVVisualizationMode::Faces)
	{
		// Flat-shaded: unshared vertices, one color per face
		int32 NumFaces = Faces.Num();
		Vertices.SetNumUninitialized(NumFaces * 3);
		Indices.SetNumUninitialized(NumFaces * 3);

		for (int32 i = 0; i < NumFaces; i++)
		{
			const FIntVector& Face = Faces[i];
			FColor Color = ChartToColor(i);

			Vertices[i * 3 + 0] = { UVs[Face.X], Color };
			Vertices[i * 3 + 1] = { UVs[Face.Y], Color };
			Vertices[i * 3 + 2] = { UVs[Face.Z], Color };

			Indices[i * 3 + 0] = static_cast<uint32>(i * 3 + 0);
			Indices[i * 3 + 1] = static_cast<uint32>(i * 3 + 1);
			Indices[i * 3 + 2] = static_cast<uint32>(i * 3 + 2);
		}

		return;
	}

	// Shared vertices: per-vertex color from mode
	TArray<FColor> VertexColors;
	VertexColors.SetNumZeroed(UVs.Num());

	if (Mode == EVEUVVisualizationMode::Charts)
	{
		bool bHasSelection = (LastHit.ChartIndex != INDEX_NONE);
		for (int32 i = 0; i < Faces.Num(); i++)
		{
			const FIntVector& Face = Faces[i];
			int32 ChartIdx = Result.FaceChartIndices.IsValidIndex(i) ? Result.FaceChartIndices[i] : INDEX_NONE;
			FColor Color = ChartToColor(ChartIdx);
			if (bHasSelection && ChartIdx != LastHit.ChartIndex)
			{
				Color.R = Color.R / 4;
				Color.G = Color.G / 4;
				Color.B = Color.B / 4;
			}
			VertexColors[Face.X] = Color;
			VertexColors[Face.Y] = Color;
			VertexColors[Face.Z] = Color;
		}
	}
	else
	{
		for (int32 i = 0; i < UVs.Num(); i++)
		{
			VertexColors[i] = UVToColor(UVs[i]);
		}
	}

	Vertices.SetNumUninitialized(UVs.Num());
	for (int32 i = 0; i < UVs.Num(); i++)
	{
		Vertices[i] = { UVs[i], VertexColors[i] };
	}

	Indices.SetNumUninitialized(Faces.Num() * 3);
	for (int32 i = 0; i < Faces.Num(); i++)
	{
		const FIntVector& Face = Faces[i];
		Indices[i * 3 + 0] = static_cast<uint32>(Face.X);
		Indices[i * 3 + 1] = static_cast<uint32>(Face.Y);
		Indices[i * 3 + 2] = static_cast<uint32>(Face.Z);
	}
}

void SVEUVMeshViewport::BuildMeshFromSnapshot(TSharedPtr<VEUV::FDebugCapture> InCapture, const VEUV::FDebugCapture::FGeometrySnapshot& Snapshot)
{
	bUniformZoom = true;
	bFlipY = false;
	bUseMaterialBrush = true;
	Capture = InCapture;

	// UV-colored shared vertices
	Vertices.SetNumUninitialized(Snapshot.VertexUVs.Num());
	for (int32 i = 0; i < Snapshot.VertexUVs.Num(); i++)
	{
		Vertices[i] = { Snapshot.VertexUVs[i], UVToColor(Snapshot.VertexUVs[i]) };
	}

	Indices.SetNumUninitialized(Snapshot.Faces.Num() * 3);
	for (int32 i = 0; i < Snapshot.Faces.Num(); i++)
	{
		const FInt32Vector3& Face = Snapshot.Faces[i];
		Indices[i * 3 + 0] = static_cast<uint32>(Face.X);
		Indices[i * 3 + 1] = static_cast<uint32>(Face.Y);
		Indices[i * 3 + 2] = static_cast<uint32>(Face.Z);
	}
}

void SVEUVMeshViewport::ClearContent()
{
	Vertices.Reset();
	Indices.Reset();
	Capture.Reset();
	LastHit = FVEUVHitResult();
	bIsDraggingSelection = false;
}

FBox2f SVEUVMeshViewport::GetContentBounds() const
{
	FVector2f Min(FLT_MAX, FLT_MAX);
	FVector2f Max(-FLT_MAX, -FLT_MAX);
	for (const FMeshVertex& V : Vertices)
	{
		Min.X = FMath::Min(Min.X, V.UV.X);
		Min.Y = FMath::Min(Min.Y, V.UV.Y);
		Max.X = FMath::Max(Max.X, V.UV.X);
		Max.Y = FMath::Max(Max.Y, V.UV.Y);
	}
	return FBox2f(Min, Max);
}

int32 SVEUVMeshViewport::PaintContent(const FGeometry& Geometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin) const
{
	// Get brush
	const FSlateBrush* Brush = (bUseMaterialBrush && GridMaterialBrush.IsValid()) ? GridMaterialBrush.Get() : FCoreStyle::Get().GetDefaultBrush();
	
	// Triangulated mesh
	TArray<FSlateVertex> SlateVerts;
	SlateVerts.SetNumUninitialized(Vertices.Num());
	for (int32 i = 0; i < Vertices.Num(); i++)
	{
		const FMeshVertex& Vertex = Vertices[i];
		FVector2f ScreenPos = UVToScreen(Vertex.UV, Scale, Origin);
		FSlateVertex& SV = SlateVerts[i];
		SV.Position.X = ScreenPos.X;
		SV.Position.Y = ScreenPos.Y;
		SV.TexCoords[0] = Vertex.UV.X;
		SV.TexCoords[1] = Vertex.UV.Y;
		SV.TexCoords[2] = 1.0f;
		SV.TexCoords[3] = 1.0f;
		SV.Color = Vertex.Color;
		SV.MaterialTexCoords = FVector2f(Vertex.UV.X, Vertex.UV.Y);
		SV.PixelSize[0] = 1;
		SV.PixelSize[1] = 1;
	}
	
	// Draw UVs
	FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*Brush);
	FSlateDrawElement::MakeCustomVerts(OutDrawElements, LayerId, ResourceHandle, SlateVerts, Indices, nullptr, 0, 0, ESlateDrawEffect::None);

	// Wireframe?
	if (bShowWireframe)
	{
		TArray<FSlateVertex> WireVerts = SlateVerts;

		// Fade based on area
		ParallelFor(Indices.Num() / 3, [&](int32 t)
		{
			const int32 i = t * 3;
			FVector2f A(WireVerts[Indices[i]].Position.X, WireVerts[Indices[i]].Position.Y);
			FVector2f B(WireVerts[Indices[i + 1]].Position.X, WireVerts[Indices[i + 1]].Position.Y);
			FVector2f C(WireVerts[Indices[i + 2]].Position.X, WireVerts[Indices[i + 2]].Position.Y);

			float ScreenArea = 0.5f * FMath::Abs((B.X - A.X) * (C.Y - A.Y) - (C.X - A.X) * (B.Y - A.Y));
			uint8 Alpha = static_cast<uint8>(FMath::Clamp(ScreenArea / 50.0f, 0.0f, 1.0f) * 255.0f);

			WireVerts[Indices[i]].Color = FColor(0, 0, 0, Alpha);
			WireVerts[Indices[i + 1]].Color = FColor(0, 0, 0, Alpha);
			WireVerts[Indices[i + 2]].Color = FColor(0, 0, 0, Alpha);
		});
		
		FSlateResourceHandle WireResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*FCoreStyle::Get().GetDefaultBrush());
		FSlateDrawElement::MakeCustomVerts(OutDrawElements, LayerId + 2, WireResourceHandle, WireVerts, Indices, nullptr, 0, 0, ESlateDrawEffect::PreMultipliedAlpha, ESlateBatchDrawFlag::Wireframe);
	}

	// Per-chart EV lines
	if (Capture.IsValid())
	{
		for (const VEUV::FDebugCapture::FChartEVDebug& EV : Capture->ChartEVs)
		{
			FVector2f Center = UVToScreen(EV.Mean, Scale, Origin);
			FVector2f End0 = UVToScreen(EV.Mean + EV.EV0, Scale, Origin);
			FVector2f End1 = UVToScreen(EV.Mean + EV.EV1, Scale, Origin);

			FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 1, FPaintGeometry(),
				{ FVector2D(Center), FVector2D(End0) }, ESlateDrawEffect::None, FLinearColor(0, 1, 1), true, 2.0f);
			FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 1, FPaintGeometry(),
				{ FVector2D(Center), FVector2D(End1) }, ESlateDrawEffect::None, FLinearColor(1, 0, 1), true, 2.0f);
		}
	}

	// Crosshair
	if (LastHit.FaceIndex != INDEX_NONE)
	{
		FVector2f HitScreen = UVToScreen(LastHit.UV, Scale, Origin);
		constexpr float Arm = 8.0f;
		FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 2, FPaintGeometry(),
			{ FVector2D(HitScreen.X - Arm, HitScreen.Y), FVector2D(HitScreen.X + Arm, HitScreen.Y) },
			ESlateDrawEffect::None, FLinearColor::White, true, 2.0f);
		FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 2, FPaintGeometry(),
			{ FVector2D(HitScreen.X, HitScreen.Y - Arm), FVector2D(HitScreen.X, HitScreen.Y + Arm) },
			ESlateDrawEffect::None, FLinearColor::White, true, 2.0f);
	}
	
	return LayerId + 3;
}

bool SVEUVMeshViewport::HitTest(const FGeometry& Geometry, FVector2f ScreenPos, FVEUVHitResult& OutHit) const
{
	if (!Capture.IsValid() || Indices.IsEmpty())
	{
		return false;
	}

	FVector2f UV = ScreenToDataUV(Geometry, ScreenPos);
	OutHit.UV = UV;

	const TArray<FIntVector>& Faces = Capture->Result.OutputMesh.Faces;
	const TArray<FVector2f>& UVs = Capture->Result.VertexUVs;

	for (int32 i = 0; i < Faces.Num(); i++)
	{
		const FIntVector& Face = Faces[i];
		FVector Bary = FMath::GetBaryCentric2D(
			FVector2D(UV), FVector2D(UVs[Face.X]), FVector2D(UVs[Face.Y]), FVector2D(UVs[Face.Z]));

		if (Bary.X >= 0.0f && Bary.Y >= 0.0f && Bary.Z >= 0.0f)
		{
			OutHit.FaceIndex = i;
			OutHit.ChartIndex = Capture->Result.FaceChartIndices.IsValidIndex(i) ? Capture->Result.FaceChartIndices[i] : INDEX_NONE;
			return true;
		}
	}
	return false;
}

FReply SVEUVMeshViewport::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2f ScreenPos(MouseEvent.GetScreenSpacePosition().X, MouseEvent.GetScreenSpacePosition().Y);
		FVEUVHitResult Hit;
		if (HitTest(MyGeometry, ScreenPos, Hit))
		{
			LastHit = Hit;
		}
		else
		{
			LastHit = FVEUVHitResult();
		}
		OnSelectionChanged.ExecuteIfBound(LastHit);
		bIsDraggingSelection = true;
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return SVEUVCanvas::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SVEUVMeshViewport::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2f ScreenPos(MouseEvent.GetScreenSpacePosition().X, MouseEvent.GetScreenSpacePosition().Y);
		FVEUVHitResult Hit;
		if (HitTest(MyGeometry, ScreenPos, Hit))
		{
			LastHit = Hit;
			OnSelectionChanged.ExecuteIfBound(LastHit);
			OnDoubleClick.ExecuteIfBound(LastHit);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SVEUVMeshViewport::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsDraggingSelection)
	{
		bIsDraggingSelection = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return SVEUVCanvas::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SVEUVMeshViewport::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsDraggingSelection)
	{
		FVector2f ScreenPos(MouseEvent.GetScreenSpacePosition().X, MouseEvent.GetScreenSpacePosition().Y);
		FVEUVHitResult Hit;
		if (HitTest(MyGeometry, ScreenPos, Hit))
		{
			LastHit = Hit;
		}
		else
		{
			LastHit = FVEUVHitResult();
		}
		OnSelectionChanged.ExecuteIfBound(LastHit);
		Invalidate(EInvalidateWidgetReason::Paint);
		return FReply::Handled();
	}
	return SVEUVCanvas::OnMouseMove(MyGeometry, MouseEvent);
}

void SVEUVPlotViewport::Construct(const FArguments& InArgs)
{
	SVEUVCanvas::Construct(SVEUVCanvas::FArguments());
}

void SVEUVPlotViewport::BuildPlot(TArray<FPlotSeries>&& InSeries)
{
	bUniformZoom = false;
	bFlipY = true;

	PlotSeries = MoveTemp(InSeries);
	HiddenSeries.Reset();
}

void SVEUVPlotViewport::ClearContent()
{
	PlotSeries.Reset();
	HiddenSeries.Reset();
	LegendEntryBounds.Reset();
}

FBox2f SVEUVPlotViewport::GetContentBounds() const
{
	FVector2f Min(FLT_MAX, FLT_MAX);
	FVector2f Max(-FLT_MAX, -FLT_MAX);
	for (const FPlotSeries& Series : PlotSeries)
	{
		if (HiddenSeries.Contains(Series.Label))
		{
			continue;
		}
		for (const FVector2f& P : Series.Points)
		{
			float Y = TransformY(P.Y);
			Min.X = FMath::Min(Min.X, P.X);
			Min.Y = FMath::Min(Min.Y, Y);
			Max.X = FMath::Max(Max.X, P.X);
			Max.Y = FMath::Max(Max.Y, Y);
		}
	}
	return FBox2f(Min, Max);
}

int32 SVEUVPlotViewport::PaintContent(const FGeometry& Geometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin) const
{
	FVector2f AbsOrigin(Geometry.AbsolutePosition.X, Geometry.AbsolutePosition.Y);
	FVector2f AbsSize(Geometry.GetAbsoluteSize().X, Geometry.GetAbsoluteSize().Y);

	// Series lines
	for (const FPlotSeries& Series : PlotSeries)
	{
		if (Series.Points.Num() < 2 || HiddenSeries.Contains(Series.Label))
		{
			continue;
		}

		TArray<FVector2D> LinePoints;
		LinePoints.Reserve(Series.Points.Num());
		for (const FVector2f& P : Series.Points)
		{
			LinePoints.Add(FVector2D(UVToScreen(FVector2f(P.X, TransformY(P.Y)), Scale, Origin)));
		}

		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, FPaintGeometry(), LinePoints,
			ESlateDrawEffect::None, Series.Color, true, 2.0f);
	}

	// Legend
	{
		LegendEntryBounds.Reset();

		FSlateFontInfo LegendFont = FCoreStyle::Get().GetFontStyle("SmallFont");
		LegendFont.Size = 10;
		constexpr float LegendPadding = 8.0f;
		constexpr float LineWidth = 20.0f;
		constexpr float EntryHeight = 16.0f;
		constexpr float LegendTotalWidth = 100.0f;

		// Local-space positioning for hit testing
		float LocalLegendX = Geometry.GetLocalSize().X - LegendTotalWidth;
		float LocalLegendY = LegendPadding;

		// Absolute-space positioning for drawing
		float LegendX = AbsOrigin.X + AbsSize.X - LegendTotalWidth;
		float LegendY = AbsOrigin.Y + LegendPadding;

		for (int32 i = 0; i < PlotSeries.Num(); i++)
		{
			const FPlotSeries& Series = PlotSeries[i];
			bool bHidden = HiddenSeries.Contains(Series.Label);
			FLinearColor DrawColor = bHidden
				? FLinearColor(Series.Color.R * 0.3f, Series.Color.G * 0.3f, Series.Color.B * 0.3f, 0.5f)
				: Series.Color;

			// Store bounds in local space for hit testing
			LegendEntryBounds.Add(FBox2f(
				FVector2f(LocalLegendX, LocalLegendY),
				FVector2f(LocalLegendX + LegendTotalWidth, LocalLegendY + EntryHeight)));

			FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 1, FPaintGeometry(),
				{ FVector2D(LegendX, LegendY + EntryHeight * 0.5), FVector2D(LegendX + LineWidth, LegendY + EntryHeight * 0.5) },
				ESlateDrawEffect::None, DrawColor, true, 3.0f);

			FSlateDrawElement::MakeText(OutDrawElements, LayerId + 1,
				FPaintGeometry(FVector2D(LegendX + LineWidth + 4.0f, LegendY), FVector2D(80, EntryHeight), 1.0f),
				Series.Label, LegendFont, ESlateDrawEffect::None, DrawColor);

			LegendY += EntryHeight + 2.0f;
			LocalLegendY += EntryHeight + 2.0f;
		}
	}

	return LayerId + 2;
}

FReply SVEUVPlotViewport::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2D LocalPos2D = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		FVector2f LocalPos(LocalPos2D.X, LocalPos2D.Y);
		for (int32 i = 0; i < LegendEntryBounds.Num() && i < PlotSeries.Num(); i++)
		{
			const FBox2f& Bounds = LegendEntryBounds[i];
			if (LocalPos.X >= Bounds.Min.X && LocalPos.X <= Bounds.Max.X &&
				LocalPos.Y >= Bounds.Min.Y && LocalPos.Y <= Bounds.Max.Y)
			{
				const FString& Label = PlotSeries[i].Label;
				if (HiddenSeries.Contains(Label))
				{
					HiddenSeries.Remove(Label);
				}
				else
				{
					HiddenSeries.Add(Label);
				}
				Invalidate(EInvalidateWidgetReason::Paint);
				bPendingFitToContent = true;
				return FReply::Handled();
			}
		}
	}

	return SVEUVCanvas::OnMouseButtonDown(MyGeometry, MouseEvent);
}

SVEUVCanvas* SVEUVDebugPanel::GetActiveCanvas() const
{
	if (CurrentMode == EVEUVVisualizationMode::ErrorPlot)
	{
		return PlotViewport.Get();
	}
	return MeshViewport.Get();
}

void SVEUVDebugPanel::Construct(const FArguments&)
{
	CaptureAddedHandle = VEUV::FDebugHistory::Get().OnCaptureAdded.AddRaw(this, &SVEUVDebugPanel::OnCaptureAdded);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[ SNew(SButton).Text(FText::FromString(TEXT("UV Layout"))).OnClicked_Lambda([this] { SetVisualizationMode(EVEUVVisualizationMode::UVLayout); return FReply::Handled(); }) ]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[ SNew(SButton).Text(FText::FromString(TEXT("Faces"))).OnClicked_Lambda([this] { SetVisualizationMode(EVEUVVisualizationMode::Faces); return FReply::Handled(); }) ]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[ SNew(SButton).Text(FText::FromString(TEXT("Charts"))).OnClicked_Lambda([this] { SetVisualizationMode(EVEUVVisualizationMode::Charts); return FReply::Handled(); }) ]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[ SNew(SButton).Text(FText::FromString(TEXT("Error"))).OnClicked_Lambda([this] { SetVisualizationMode(EVEUVVisualizationMode::ErrorPlot); return FReply::Handled(); }) ]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[ SNew(SButton).Text(FText::FromString(TEXT("Stats"))).OnClicked_Lambda([this] { SetVisualizationMode(EVEUVVisualizationMode::Stats); return FReply::Handled(); }) ]
			+ SHorizontalBox::Slot().AutoWidth().Padding(2)
			[
				SNew(SButton).Text(FText::FromString(TEXT("Clear"))).OnClicked_Lambda([this]
				{
					VEUV::FDebugHistory::Get().Clear();
					RefreshCaptureList();
					MeshViewport->Clear();
					PlotViewport->Clear();
					SelectedCaptureIndex = INDEX_NONE;
					StatsText->SetText(FText::FromString(TEXT("Cleared")));
					DrawWorldDebug();
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot().FillWidth(1.0f) [ SNullWidget::NullWidget ]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(0, 0, 4, 0)
				[ SNew(STextBlock).Text(FText::FromString(TEXT("Samples:"))) ]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SComboButton)
					.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
					{
						FMenuBuilder MenuBuilder(true, nullptr);
						auto AddEntry = [&](const FText& Label, ESampleFilter Filter)
						{
							MenuBuilder.AddMenuEntry(Label, FText(), FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([this, Filter]
									{
										SampleFilter = Filter;
										DrawWorldDebug();
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([this, Filter]
									{
										return SampleFilter == Filter;
									})
								),
								NAME_None, EUserInterfaceActionType::RadioButton);
						};
							
						AddEntry(NSLOCTEXT("VEUV", "SamplesNone", "None"), ESampleFilter::None);
						AddEntry(NSLOCTEXT("VEUV", "SamplesAll", "All"), ESampleFilter::All);
						AddEntry(NSLOCTEXT("VEUV", "SamplesArea", "Area"), ESampleFilter::Area);
						AddEntry(NSLOCTEXT("VEUV", "SamplesComplexity", "Complexity"), ESampleFilter::Complexity);
						AddEntry(NSLOCTEXT("VEUV", "SamplesAdaptive", "Adaptive"), ESampleFilter::Adaptive);

						return MenuBuilder.MakeWidget();
					})
					.ButtonContent()
					[
						SNew(STextBlock).Text_Lambda([this]
						{
							switch (SampleFilter)
							{
							case ESampleFilter::None:       return FText::FromString(TEXT("None"));
							case ESampleFilter::All:        return FText::FromString(TEXT("All"));
							case ESampleFilter::Area:       return FText::FromString(TEXT("Area"));
							case ESampleFilter::Complexity: return FText::FromString(TEXT("Complexity"));
							case ESampleFilter::Adaptive:   return FText::FromString(TEXT("Adaptive"));
							default:                        return FText::FromString(TEXT("?"));
							}
						})
					]
				]
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return bShowVoxels ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bShowVoxels = (State == ECheckBoxState::Checked); 
					DrawWorldDebug();
				})
				[SNew(STextBlock).Text(FText::FromString(TEXT("Show Voxels")))]
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return MeshViewport.IsValid() && MeshViewport->bShowWireframe ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					if (MeshViewport.IsValid())
					{
						MeshViewport->bShowWireframe = (State == ECheckBoxState::Checked);
						MeshViewport->Invalidate(EInvalidateWidgetReason::Paint);
					}
				})
				[SNew(STextBlock).Text(FText::FromString(TEXT("Wireframe")))]
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[
				SNew(SButton).Text(FText::FromString(TEXT("Settings"))).OnClicked_Lambda([this]
				{
					bShowSettings = !bShowSettings;
					if (SettingsPanel.IsValid())
					{
						SettingsPanel->SetVisibility(bShowSettings ? EVisibility::Visible : EVisibility::Collapsed);
					}
					return FReply::Handled();
				})
			]
		]

		// Build selector
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(2)
			[ SNew(STextBlock).Text(FText::FromString(TEXT("Build:"))) ]
			+ SHorizontalBox::Slot().FillWidth(1.0f).Padding(2)
			[
				SAssignNew(BuildComboBox, SComboBox<TSharedPtr<int32>>)
				.OptionsSource(&ComboBoxSource)
				.OnGenerateWidget(this, &SVEUVDebugPanel::GenerateComboBoxRow)
				.OnSelectionChanged(this, &SVEUVDebugPanel::OnComboBoxSelectionChanged)
				[ SNew(STextBlock).Text(this, &SVEUVDebugPanel::GetSelectedComboBoxText) ]
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[ SNew(STextBlock).Text(FText::FromString(TEXT("Snapshot:"))) ]
			+ SHorizontalBox::Slot().FillWidth(0.5f).Padding(2)
			[
				SAssignNew(SnapshotComboBox, SComboBox<TSharedPtr<int32>>)
				.OptionsSource(&SnapshotComboBoxSource)
				.OnGenerateWidget_Lambda([this](TSharedPtr<int32> InItem) -> TSharedRef<SWidget>
				{
					int32 Idx = *InItem;
						
					FString Label;
					if (Idx == INDEX_NONE)
					{
						Label = TEXT("(Result)");
					}
					else if (Captures.IsValidIndex(SelectedCaptureIndex))
					{
						const TArray<VEUV::FDebugCapture::FGeometrySnapshot>& Snapshots = Captures[SelectedCaptureIndex]->Snapshots;
						Label = Snapshots.IsValidIndex(Idx) ? Snapshots[Idx].Name : TEXT("?");
					}
						
					return SNew(STextBlock).Text(FText::FromString(Label));
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<int32> InItem, ESelectInfo::Type)
				{
					if (InItem.IsValid())
					{
						SelectedSnapshotIndex = *InItem;
						RebuildMeshData();
					}
				})
				[
					SNew(STextBlock).Text_Lambda([this]
					{
						if (SelectedSnapshotIndex == INDEX_NONE)
						{
							return FText::FromString(TEXT("(Result)"));
						}
						if (Captures.IsValidIndex(SelectedCaptureIndex))
						{
							const TArray<VEUV::FDebugCapture::FGeometrySnapshot>& Snapshots = Captures[SelectedCaptureIndex]->Snapshots;
							if (Snapshots.IsValidIndex(SelectedSnapshotIndex))
							{
								return FText::FromString(Snapshots[SelectedSnapshotIndex].Name);
							}
						}
							
						return FText::FromString(TEXT("?"));
					})
				]
			]
			+ SHorizontalBox::Slot().FillWidth(0.5f).VAlign(VAlign_Center).Padding(2)
			[
				SNew(SSlider)
				.Value_Lambda([this]
				{
					if (!Captures.IsValidIndex(SelectedCaptureIndex) || Captures[SelectedCaptureIndex]->Snapshots.IsEmpty())
					{
						return 0.0f;
					}
					return FMath::Clamp(static_cast<float>(SelectedSnapshotIndex) / FMath::Max(1, Captures[SelectedCaptureIndex]->Snapshots.Num() - 1), 0.0f, 1.0f);
				})
				.OnValueChanged_Lambda([this](float Value)
				{
					if (Captures.IsValidIndex(SelectedCaptureIndex) && !Captures[SelectedCaptureIndex]->Snapshots.IsEmpty())
					{
						SelectedSnapshotIndex = FMath::RoundToInt32(Value * (Captures[SelectedCaptureIndex]->Snapshots.Num() - 1));
						RebuildMeshData();
					}
				})
				.Visibility_Lambda([this]
				{
					return (Captures.IsValidIndex(SelectedCaptureIndex) && Captures[SelectedCaptureIndex]->Snapshots.Num() > 0)
						? EVisibility::Visible : EVisibility::Collapsed;
				})
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(2)
			[
				SNew(SComboButton)
				.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
				{
					FMenuBuilder MenuBuilder(true, nullptr);
					for (float FPS : { 8.0f, 16.0f, 24.0f, 32.0f, 64.0f, 128.0f })
					{
						MenuBuilder.AddMenuEntry(
							FText::FromString(FString::Printf(TEXT("%.0f fps"), FPS)),
							FText(), FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this, FPS]
								{
									PlaybackFPS = FPS;
								}),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([this, FPS]
								{
									return PlaybackFPS == FPS;
								})
							),
							NAME_None, EUserInterfaceActionType::RadioButton);
					}
					return MenuBuilder.MakeWidget();
				})
				.ButtonContent()
				[
					SNew(STextBlock).Text_Lambda([this]
					{
						return FText::FromString(FString::Printf(TEXT("%.0f fps"), PlaybackFPS));
					})
				]
				.Visibility_Lambda([this]
				{
					return (Captures.IsValidIndex(SelectedCaptureIndex) && Captures[SelectedCaptureIndex]->Snapshots.Num() > 0)
						? EVisibility::Visible : EVisibility::Collapsed;
				})
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(2)
			[
				SNew(SButton)
				.Text_Lambda([this] { return FText::FromString(bPlayingSnapshots ? TEXT("Stop") : TEXT("Play")); })
				.OnClicked_Lambda([this]
				{
					bPlayingSnapshots = !bPlayingSnapshots;
					if (bPlayingSnapshots)
					{
						PlaybackAccumulator = 0.0f;
						
						bool bEOP = Captures.IsValidIndex(SelectedCaptureIndex) && SelectedSnapshotIndex >= Captures[SelectedCaptureIndex]->Snapshots.Num() - 1;
						if (SelectedSnapshotIndex == INDEX_NONE || bEOP)
						{
							SelectedSnapshotIndex = 0;
						}
						
						RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SVEUVDebugPanel::OnPlaybackTick));
					}
						
					return FReply::Handled();
				})
				.Visibility_Lambda([this]
				{
					return (Captures.IsValidIndex(SelectedCaptureIndex) && Captures[SelectedCaptureIndex]->Snapshots.Num() > 0)
						? EVisibility::Visible : EVisibility::Collapsed;
				})
			]
		]

		// Settings panel
		+ SVerticalBox::Slot().AutoHeight().MaxHeight(300).Padding(4)
		[ SAssignNew(SettingsPanel, SBox) ]

		// Viewport container (swaps between mesh and plot)
		+ SVerticalBox::Slot().FillHeight(1.0f).Padding(4)
		[
			SAssignNew(ViewportContainer, SBox)
		]

		// Status line
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().FillWidth(1.0f)
			[
				SAssignNew(StatsText, STextBlock)
				.Text(FText::FromString(TEXT("Enable VEUV.Debug.CaptureEnabled to capture builds")))
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(4, 0)
			[
				SAssignNew(LogScaleCheckbox, SCheckBox)
				.IsChecked_Lambda([this]
				{
					return PlotViewport.IsValid() && PlotViewport->bLogScale ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					if (PlotViewport.IsValid())
					{
						PlotViewport->bLogScale = (State == ECheckBoxState::Checked);
						PlotViewport->RequestFitToContent();
						PlotViewport->Invalidate(EInvalidateWidgetReason::Paint);
					}
				})
				.Visibility_Lambda([this]
				{
					return CurrentMode == EVEUVVisualizationMode::ErrorPlot ? EVisibility::Visible : EVisibility::Collapsed;
				})
				[SNew(STextBlock).Text(FText::FromString(TEXT("Log")))]
			]
		]
	];

	// Create both viewports (only one visible at a time)
	SAssignNew(MeshViewport, SVEUVMeshViewport);
	SAssignNew(PlotViewport, SVEUVPlotViewport);

	// Default to mesh viewport
	ViewportContainer->SetContent(MeshViewport.ToSharedRef());

	// Settings panel
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		SettingsStructData = MakeShared<FStructOnScope>(FVEUVConfig::StaticStruct(), reinterpret_cast<uint8*>(&FVEUVConfig::Default));

		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = true;
		ViewArgs.bShowPropertyMatrixButton = false;
		ViewArgs.bHideSelectionTip = true;
		ViewArgs.bShowScrollBar = true;

		FStructureDetailsViewArgs StructArgs;
		StructArgs.bShowObjects = true;

		SettingsDetailsView = PropertyModule.CreateStructureDetailView(ViewArgs, StructArgs, SettingsStructData);
		SettingsDetailsView->GetDetailsView()->SetRootExpansionStates(true, true);
		SettingsPanel->SetContent(SettingsDetailsView->GetWidget().ToSharedRef());
	}
	SettingsPanel->SetVisibility(EVisibility::Collapsed);

	// View change callback (bind to both viewports)
	auto ViewChangedHandler = [this]
	{
		SVEUVCanvas* Active = GetActiveCanvas();
		if (StatsText.IsValid() && Active)
		{
			StatsText->SetText(FText::FromString(FString::Printf(
				TEXT("Zoom: (%.3f, %.3f)  Offset: (%.3f, %.3f)"),
				Active->GetViewZoom().X, Active->GetViewZoom().Y,
				Active->GetViewOffset().X, Active->GetViewOffset().Y)));
		}
	};
	MeshViewport->OnViewChanged.BindLambda(ViewChangedHandler);
	PlotViewport->OnViewChanged.BindLambda(ViewChangedHandler);

	// Selection callback (mesh only)
	MeshViewport->OnSelectionChanged.BindLambda([this](const FVEUVHitResult& Hit)
	{
		if (CurrentMode == EVEUVVisualizationMode::Charts && Hit.ChartIndex != LastSelectedChart)
		{
			RebuildMeshData();
			LastSelectedChart = Hit.ChartIndex;
		}
		DrawWorldDebug();
		if (GEditor)
		{
			for (FLevelEditorViewportClient* VC : GEditor->GetLevelViewportClients())
			{
				if (VC) { VC->Invalidate(); }
			}
		}
	});

	// Double click (mesh only)
	MeshViewport->OnDoubleClick.BindLambda([this](const FVEUVHitResult& Hit)
	{
		if (Hit.FaceIndex == INDEX_NONE || !Captures.IsValidIndex(SelectedCaptureIndex))
		{
			return;
		}

		const TSharedPtr<VEUV::FDebugCapture>& Capture = Captures[SelectedCaptureIndex];
		const FIntVector& Face = Capture->Result.OutputMesh.Faces[Hit.FaceIndex];

		FVector A(Capture->Result.OutputMesh.Vertices[Face.X]);
		FVector B(Capture->Result.OutputMesh.Vertices[Face.Y]);
		FVector C(Capture->Result.OutputMesh.Vertices[Face.Z]);

		FVector Bary = FMath::GetBaryCentric2D(
			FVector2D(Hit.UV),
			FVector2D(Capture->Result.VertexUVs[Face.X]),
			FVector2D(Capture->Result.VertexUVs[Face.Y]),
			FVector2D(Capture->Result.VertexUVs[Face.Z]));

		FVector WorldPos = Capture->Config.DebugWorldTransform.TransformPosition(A * Bary.X + B * Bary.Y + C * Bary.Z);

		if (GEditor)
		{
			for (FLevelEditorViewportClient* VC : GEditor->GetLevelViewportClients())
			{
				if (VC && VC->IsPerspective())
				{
					VC->SetViewLocationForOrbiting(WorldPos, 512.0f);
					VC->Invalidate();
					break;
				}
			}
		}
	});

	RefreshCaptureList();
	if (Captures.Num() > 0)
	{
		SelectCapture(Captures.Num() - 1);
	}
}

SVEUVDebugPanel::~SVEUVDebugPanel()
{
	VEUV::FDebugHistory::Get().OnCaptureAdded.Remove(CaptureAddedHandle);

	UWorld* DebugWorld = GEngine ? GEngine->GetWorldContexts()[0].World() : nullptr;
	if (DebugWorld)
	{
		FlushPersistentDebugLines(DebugWorld);
	}
}

void SVEUVDebugPanel::OnCaptureAdded()
{
	TWeakPtr<SVEUVDebugPanel> WeakThis = SharedThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis]
	{
		TSharedPtr<SVEUVDebugPanel> This = WeakThis.Pin();
		if (!This) { return; }
		This->RefreshCaptureList();
		if (This->Captures.Num() > 0)
		{
			int32 Last = This->Captures.Num() - 1;
			This->SelectCapture(Last);
			if (This->BuildComboBox.IsValid() && This->ComboBoxSource.IsValidIndex(Last))
			{
				This->BuildComboBox->SetSelectedItem(This->ComboBoxSource[Last]);
			}
		}
	});
}

void SVEUVDebugPanel::RefreshCaptureList()
{
	Captures = VEUV::FDebugHistory::Get().GetCaptures();
	RebuildComboBoxSource();
}

void SVEUVDebugPanel::RebuildComboBoxSource()
{
	ComboBoxSource.Reset();
	for (int32 i = 0; i < Captures.Num(); i++)
	{
		ComboBoxSource.Add(MakeShared<int32>(i));
	}
	if (BuildComboBox.IsValid())
	{
		BuildComboBox->RefreshOptions();
	}
}

void SVEUVDebugPanel::OnComboBoxSelectionChanged(TSharedPtr<int32> InItem, ESelectInfo::Type)
{
	if (InItem.IsValid())
	{
		SelectCapture(*InItem);
	}
}

TSharedRef<SWidget> SVEUVDebugPanel::GenerateComboBoxRow(TSharedPtr<int32> InItem)
{
	int32 CaptureIndex = *InItem;
	FString Label;
	if (Captures.IsValidIndex(CaptureIndex))
	{
		const TSharedPtr<VEUV::FDebugCapture>& Capture = Captures[CaptureIndex];
		Label = FString::Printf(TEXT("#%d - %s (%d charts, %.0f ms)"),
			CaptureIndex, *Capture->Timestamp.ToString(TEXT("%H:%M:%S")),
			Capture->Result.Charts.Num(), Capture->Result.Stats.TotalMs);
	}
	return SNew(STextBlock).Text(FText::FromString(Label));
}

FText SVEUVDebugPanel::GetSelectedComboBoxText() const
{
	if (Captures.IsValidIndex(SelectedCaptureIndex))
	{
		const TSharedPtr<VEUV::FDebugCapture>& Capture = Captures[SelectedCaptureIndex];
		return FText::FromString(FString::Printf(TEXT("#%d - %s (%d charts, %.0f ms)"),
			SelectedCaptureIndex, *Capture->Timestamp.ToString(TEXT("%H:%M:%S")),
			Capture->Result.Charts.Num(), Capture->Result.Stats.TotalMs));
	}
	return FText::FromString(TEXT("No captures"));
}

void SVEUVDebugPanel::SelectCapture(int32 Index)
{
	SelectedCaptureIndex = Index;
	SelectedSnapshotIndex = INDEX_NONE;
	MeshViewport->LastHit = FVEUVHitResult();

	// Rebuild snapshot combobox options
	SnapshotComboBoxSource.Reset();
	SnapshotComboBoxSource.Add(MakeShared<int32>(INDEX_NONE));
	
	if (Captures.IsValidIndex(Index))
	{
		for (int32 i = 0; i < Captures[Index]->Snapshots.Num(); i++)
		{
			SnapshotComboBoxSource.Add(MakeShared<int32>(i));
		}
	}
	
	if (SnapshotComboBox.IsValid())
	{
		SnapshotComboBox->RefreshOptions();
		SnapshotComboBox->SetSelectedItem(SnapshotComboBoxSource[0]);
	}

	RebuildMeshData();
	if (SVEUVCanvas* Active = GetActiveCanvas())
	{
		Active->RequestFitToContent();
	}
	DrawWorldDebug();
}

void SVEUVDebugPanel::SetVisualizationMode(EVEUVVisualizationMode Mode)
{
	CurrentMode = Mode;

	// Swap active viewport
	if (Mode == EVEUVVisualizationMode::ErrorPlot)
	{
		ViewportContainer->SetContent(PlotViewport.ToSharedRef());
	}
	else
	{
		ViewportContainer->SetContent(MeshViewport.ToSharedRef());
	}

	RebuildMeshData();
	if (SVEUVCanvas* Active = GetActiveCanvas())
	{
		Active->RequestFitToContent();
	}
	DrawWorldDebug();
}

void SVEUVDebugPanel::RebuildMeshData()
{
	if (!Captures.IsValidIndex(SelectedCaptureIndex))
	{
		MeshViewport->Clear();
		PlotViewport->Clear();
		StatsText->SetText(FText::FromString(TEXT("No capture selected")));
		return;
	}

	const TSharedPtr<VEUV::FDebugCapture>& Capture = Captures[SelectedCaptureIndex];

	if (CurrentMode == EVEUVVisualizationMode::Stats)
	{
		MeshViewport->Clear();
		StatsText->SetText(FText::FromString(Capture->Result.Stats.ToString()));
		return;
	}

	if (CurrentMode == EVEUVVisualizationMode::ErrorPlot)
	{
		TArray<SVEUVPlotViewport::FPlotSeries> Series;

		if (!Capture->R78ErrorHistory.IsEmpty())
		{
			SVEUVPlotViewport::FPlotSeries& R78 = Series.AddDefaulted_GetRef();
			R78.Label = TEXT("R78");
			R78.Color = FLinearColor(0.0f, 0.8f, 1.0f);
			for (int32 i = 0; i < Capture->R78ErrorHistory.Num(); i++)
			{
				R78.Points.Add(FVector2f(static_cast<float>(i), Capture->R78ErrorHistory[i]));
			}
		}

		if (!Capture->R9ErrorHistory.IsEmpty())
		{
			SVEUVPlotViewport::FPlotSeries& R9 = Series.AddDefaulted_GetRef();
			R9.Label = TEXT("R9");
			R9.Color = FLinearColor(1.0f, 0.5f, 0.0f);
			for (int32 i = 0; i < Capture->R9ErrorHistory.Num(); i++)
			{
				R9.Points.Add(FVector2f(static_cast<float>(i), Capture->R9ErrorHistory[i]));
			}
		}

		if (!Capture->R9GradNormHistory.IsEmpty())
		{
			SVEUVPlotViewport::FPlotSeries& R9Loss = Series.AddDefaulted_GetRef();
			R9Loss.Label = TEXT("R9-GradNorm");
			R9Loss.Color = FLinearColor(1.0f, 0.0f, 1.0f);
			for (int32 i = 0; i < Capture->R9GradNormHistory.Num(); i++)
			{
				R9Loss.Points.Add(FVector2f(static_cast<float>(i), Capture->R9GradNormHistory[i]));
			}
		}

		if (!Capture->R10GradNormHistory.IsEmpty())
		{
			SVEUVPlotViewport::FPlotSeries& R10Loss = Series.AddDefaulted_GetRef();
			R10Loss.Label = TEXT("R10-GradNorm");
			R10Loss.Color = FLinearColor(1.0f, 1.0f, 0.0f);
			for (int32 i = 0; i < Capture->R10GradNormHistory.Num(); i++)
			{
				R10Loss.Points.Add(FVector2f(static_cast<float>(i), Capture->R10GradNormHistory[i]));
			}
		}

		PlotViewport->BuildPlot(MoveTemp(Series));
		StatsText->SetText(FText::FromString(FString::Printf(
			TEXT("R78: %d points, R9: %d points, R9Grad: %d points, R10Grad: %d points"),
			Capture->R78ErrorHistory.Num(), Capture->R9ErrorHistory.Num(),
			Capture->R9GradNormHistory.Num(), Capture->R10GradNormHistory.Num())));
		return;
	}

	// Use snapshot if selected, otherwise current result
	if (SelectedSnapshotIndex != INDEX_NONE && Capture->Snapshots.IsValidIndex(SelectedSnapshotIndex))
	{
		MeshViewport->BuildMeshFromSnapshot(Capture, Capture->Snapshots[SelectedSnapshotIndex]);
		StatsText->SetText(FText::FromString(FString::Printf(
			TEXT("Snapshot: %s (%d faces)"),
			*Capture->Snapshots[SelectedSnapshotIndex].Name,
			Capture->Snapshots[SelectedSnapshotIndex].Faces.Num()
		)));
	}
	else
	{
		MeshViewport->BuildMesh(Capture, CurrentMode);
		StatsText->SetText(FText::FromString(FString::Printf(
			TEXT("%d faces, %d charts, %.1f ms total"),
			Capture->Result.OutputMesh.Faces.Num(), Capture->Result.Charts.Num(), Capture->Result.Stats.TotalMs
		)));
	}
}

EActiveTimerReturnType SVEUVDebugPanel::OnPlaybackTick(double InCurrentTime, float InDeltaTime)
{
	// Invalid capture?
	if (!bPlayingSnapshots || !Captures.IsValidIndex(SelectedCaptureIndex))
	{
		bPlayingSnapshots = false;
		return EActiveTimerReturnType::Stop;
	}

	// Invalid snapshot set?
	const TArray<VEUV::FDebugCapture::FGeometrySnapshot>& Snapshots = Captures[SelectedCaptureIndex]->Snapshots;
	if (Snapshots.IsEmpty())
	{
		bPlayingSnapshots = false;
		return EActiveTimerReturnType::Stop;
	}

	// Advance by delta
	PlaybackAccumulator += InDeltaTime;
	float FrameInterval = 1.0f / FMath::Max(1.0f, PlaybackFPS);
	int32 FramesToAdvance = FMath::FloorToInt32(PlaybackAccumulator / FrameInterval);
	PlaybackAccumulator -= FramesToAdvance * FrameInterval;

	if (FramesToAdvance <= 0)
	{
		return EActiveTimerReturnType::Continue;
	}

	// Out of snapshots?
	SelectedSnapshotIndex = FMath::Min(SelectedSnapshotIndex + FramesToAdvance, Snapshots.Num() - 1);
	if (SelectedSnapshotIndex >= Snapshots.Num() - 1)
	{
		SelectedSnapshotIndex = Snapshots.Num() - 1;
		bPlayingSnapshots = false;
		RebuildMeshData();
		return EActiveTimerReturnType::Stop;
	}

	// Update combobox selection
	if (SnapshotComboBox.IsValid() && SnapshotComboBoxSource.IsValidIndex(SelectedSnapshotIndex + 1))
	{
		SnapshotComboBox->SetSelectedItem(SnapshotComboBoxSource[SelectedSnapshotIndex + 1]);
	}

	RebuildMeshData();
	return EActiveTimerReturnType::Continue;
}

void SVEUVDebugPanel::DrawWorldDebug()
{
	TWeakPtr<SVEUVDebugPanel> WeakThis = SharedThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis]
	{
		TSharedPtr<SVEUVDebugPanel> This = WeakThis.Pin();
		if (!This) { return; }

		UWorld* DebugWorld = GEngine ? GEngine->GetWorldContexts()[0].World() : nullptr;
		if (!DebugWorld)
		{
			return;
		}

		FlushPersistentDebugLines(DebugWorld);

		if (!This->Captures.IsValidIndex(This->SelectedCaptureIndex))
		{
			return;
		}

		const TSharedPtr<VEUV::FDebugCapture>& Capture = This->Captures[This->SelectedCaptureIndex];

		if (This->SampleFilter != ESampleFilter::None)
		{
			for (const VEUV::FDebugCapture::FDebugSample& S : Capture->Samples)
			{
				bool bShow = false;
				switch (This->SampleFilter)
				{
				case ESampleFilter::All:        bShow = true; break;
				case ESampleFilter::Area:       bShow = (S.Type == EVEUVSampleType::Area); break;
				case ESampleFilter::Complexity: bShow = (S.Type == EVEUVSampleType::Complexity); break;
				case ESampleFilter::Adaptive:   bShow = (S.Type == EVEUVSampleType::Adaptive); break;
				default: break;
				}

				if (bShow)
				{
					DrawDebugPoint(DebugWorld, S.Position, 6.5f, S.Color, true, -1.0f, SDPG_Foreground);
				}
			}
		}

		if (This->bShowVoxels)
		{
			if (This->CurrentMode == EVEUVVisualizationMode::Charts)
			{
				int32 SelectedChart = This->MeshViewport->LastHit.ChartIndex;
				bool bHasSelection = (SelectedChart != INDEX_NONE);

				for (int32 ChartIdx = 0; ChartIdx < Capture->Result.Charts.Num(); ChartIdx++)
				{
					if (bHasSelection && ChartIdx != SelectedChart) { continue; }

					FColor ChartColor = ChartToColor(ChartIdx);
					for (int32 VoxelIndex : Capture->Result.Charts[ChartIdx].VoxelIndices)
					{
						const VEUV::FDebugCapture::FVoxelDebug& Voxel = Capture->Voxels[VoxelIndex];
						DrawDebugSolidBox(DebugWorld, Voxel.Center, Voxel.Extent, FColor(ChartColor.R, ChartColor.G, ChartColor.B, 32), true);
						DrawDebugBox(DebugWorld, Voxel.Center, Voxel.Extent, ChartColor, true, -1.0f, bHasSelection ? SDPG_Foreground : 0);
					}
				}
			}
			else
			{
				for (const VEUV::FDebugCapture::FVoxelDebug& Voxel : Capture->Voxels)
				{
					DrawDebugBox(DebugWorld, Voxel.Center, Voxel.Extent, FColor(180, 180, 180), true);
				}
			}
		}

		const FVEUVHitResult& Hit = This->MeshViewport->LastHit;
		if (Hit.FaceIndex != INDEX_NONE && Hit.FaceIndex < Capture->Result.OutputMesh.Faces.Num())
		{
			const FIntVector& Face = Capture->Result.OutputMesh.Faces[Hit.FaceIndex];

			FVector A(Capture->Result.OutputMesh.Vertices[Face.X]);
			FVector B(Capture->Result.OutputMesh.Vertices[Face.Y]);
			FVector C(Capture->Result.OutputMesh.Vertices[Face.Z]);

			FVector Bary = FMath::GetBaryCentric2D(
				FVector2D(Hit.UV),
				FVector2D(Capture->Result.VertexUVs[Face.X]),
				FVector2D(Capture->Result.VertexUVs[Face.Y]),
				FVector2D(Capture->Result.VertexUVs[Face.Z]));

			FVector WorldPos = Capture->Config.DebugWorldTransform.TransformPosition(A * Bary.X + B * Bary.Y + C * Bary.Z);

			constexpr float LineLen = 10000.0f;
			DrawDebugLine(DebugWorld, WorldPos - FVector(LineLen, 0, 0), WorldPos + FVector(LineLen, 0, 0), FColor(255, 80, 80), true, -1.0f, SDPG_Foreground, 5.5f);
			DrawDebugLine(DebugWorld, WorldPos - FVector(0, LineLen, 0), WorldPos + FVector(0, LineLen, 0), FColor(80, 255, 80), true, -1.0f, SDPG_Foreground, 5.5f);
			DrawDebugLine(DebugWorld, WorldPos - FVector(0, 0, LineLen), WorldPos + FVector(0, 0, LineLen), FColor(80, 80, 255), true, -1.0f, SDPG_Foreground, 5.5f);
			DrawDebugSphere(DebugWorld, WorldPos, 24.0f, 12, FColor::White, true, -1.0f, SDPG_Foreground, 5.0f);
		}
	});
}
