// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "VEUV/VEUVDebugCapture.h"
#include "IStructureDetailsView.h"
#include "SlateMaterialBrush.h"
#include <cmath>

enum class EVEUVVisualizationMode : uint8
{
	UVLayout,
	Faces,
	Charts,
	ErrorPlot,
	Stats
};

class SVEUVCanvas : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SVEUVCanvas) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Reset view and content */
	void Clear();

	/** Fit the view to content bounds */
	void FitToContent();

	/** Deferred fit */
	void RequestFitToContent() { bPendingFitToContent = true; }

	/** View state accessors */
	FVector2f GetViewOffset() const { return ViewOffset; }
	FVector2f GetViewZoom() const { return ViewZoom; }

	DECLARE_DELEGATE(FOnViewChanged);
	FOnViewChanged OnViewChanged;

	/** SWidget overrides */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

protected:
	/** Override to provide content bounds in data space */
	virtual FBox2f GetContentBounds() const { return FBox2f(FVector2f::ZeroVector, FVector2f::ZeroVector); }

	/** Override to render content on top of the grid */
	virtual int32 PaintContent(const FGeometry& Geometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin) const { return LayerId; }

	/** Override to clear derived content */
	virtual void ClearContent() {}

	/** Returns true if there is content to display */
	virtual bool HasContent() const { return false; }

	/** Transform data coordinate to screen position */
	FVector2f UVToScreen(FVector2f UV, FVector2f Scale, FVector2f Origin) const
	{
		float Y = bFlipY ? -UV.Y : UV.Y;
		return FVector2f(
			(UV.X * ViewZoom.X + ViewOffset.X) * Scale.X + Origin.X,
			(Y * ViewZoom.Y + ViewOffset.Y) * Scale.Y + Origin.Y);
	}

	/** Get the pixel-to-normalized scale factor */
	FVector2f GetScale(const FGeometry& Geometry) const;

	/** Convert screen position to normalized viewport coordinates */
	FVector2f ScreenToNormalized(const FGeometry& Geometry, FVector2f ScreenPos) const;

	/** Convert screen position to data-space UV coordinates */
	FVector2f ScreenToDataUV(const FGeometry& Geometry, FVector2f ScreenPos) const;

	/** View state */
	FVector2f ViewOffset = FVector2f(0.0f, 0.0f);
	FVector2f ViewZoom = FVector2f(1.0f, 1.0f);

	/** Pan state */
	FVector2f PanStart = FVector2f::ZeroVector;
	FVector2f PanOffsetStart = FVector2f::ZeroVector;
	bool bIsPanning = false;

	/** View configuration (set by derived classes during Build) */
	bool bUniformZoom = true;
	bool bFlipY = false;

	/** Deferred fit-to-content (executed on next Tick when geometry is valid) */
	bool bPendingFitToContent = false;

private:
	/** SLeafWidget overrides */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	/** Draw the background grid, axes, and tick labels */
	int32 DrawGrid(FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin, FVector2f AbsOrigin, FVector2f ViewportSize) const;

	/** Whether to draw the UV 0-1 unit box (overridden by mesh viewport) */
	virtual bool ShouldDrawUnitBox() const { return false; }
};

struct FVEUVHitResult
{
	int32 FaceIndex = INDEX_NONE;
	int32 ChartIndex = INDEX_NONE;
	FVector2f UV;
};

class SVEUVMeshViewport : public SVEUVCanvas
{
public:
	struct FMeshVertex
	{
		FVector2f UV;
		FColor Color;
	};

	SLATE_BEGIN_ARGS(SVEUVMeshViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Build mesh from a debug capture */
	void BuildMesh(TSharedPtr<VEUV::FDebugCapture> InCapture, EVEUVVisualizationMode Mode);

	/** Build mesh from a geometry snapshot */
	void BuildMeshFromSnapshot(TSharedPtr<VEUV::FDebugCapture> InCapture, const VEUV::FDebugCapture::FGeometrySnapshot& Snapshot);

	/** Find the face and chart at a screen position */
	bool HitTest(const FGeometry& Geometry, FVector2f ScreenPos, FVEUVHitResult& OutHit) const;

	/** SWidget overrides */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, const FVEUVHitResult&);
	FOnSelectionChanged OnSelectionChanged;

	DECLARE_DELEGATE_OneParam(FOnDoubleClick, const FVEUVHitResult&);
	FOnDoubleClick OnDoubleClick;

	FVEUVHitResult LastHit;

	bool bShowWireframe = false;
	
protected:
	virtual FBox2f GetContentBounds() const override;
	virtual int32 PaintContent(const FGeometry& Geometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin) const override;
	virtual void ClearContent() override;
	virtual bool HasContent() const override { return !Vertices.IsEmpty(); }
	virtual bool ShouldDrawUnitBox() const override { return true; }

private:
	TArray<FMeshVertex> Vertices;
	TArray<uint32> Indices;
	TSharedPtr<VEUV::FDebugCapture> Capture;
	bool bIsDraggingSelection = false;

	/** Optional material brush for UV grid visualization */
	TSharedPtr<FSlateMaterialBrush> GridMaterialBrush;
	bool bUseMaterialBrush = false;
};

class SVEUVPlotViewport : public SVEUVCanvas
{
public:
	struct FPlotSeries
	{
		TArray<FVector2f> Points;
		FLinearColor Color;
		FString Label;
	};

	SLATE_BEGIN_ARGS(SVEUVPlotViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Build plot from series data */
	void BuildPlot(TArray<FPlotSeries>&& InSeries);

	/** SWidget overrides */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Log scale Y axis */
	bool bLogScale = true;

protected:
	virtual FBox2f GetContentBounds() const override;
	virtual int32 PaintContent(const FGeometry& Geometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, FVector2f Scale, FVector2f Origin) const override;
	virtual void ClearContent() override;
	virtual bool HasContent() const override { return !PlotSeries.IsEmpty(); }

private:
	/** Transform Y */
	float TransformY(float Y) const
	{
		constexpr float Knee = 1000.0f;
		return bLogScale ? Knee * std::log1p(Y / Knee) : Y;
	}

	TArray<FPlotSeries> PlotSeries;
	TSet<FString> HiddenSeries;
	mutable TArray<FBox2f> LegendEntryBounds;
};

class SVEUVDebugPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVEUVDebugPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SVEUVDebugPanel();

private:
	void OnCaptureAdded();
	void RefreshCaptureList();
	void RebuildComboBoxSource();
	void SelectCapture(int32 Index);
	void SetVisualizationMode(EVEUVVisualizationMode Mode);
	void RebuildMeshData();
	void DrawWorldDebug();
	
	EActiveTimerReturnType OnPlaybackTick(double InCurrentTime, float InDeltaTime);

	/** Get whichever canvas is currently active */
	SVEUVCanvas* GetActiveCanvas() const;

	TSharedRef<SWidget> GenerateComboBoxRow(TSharedPtr<int32> InItem);
	void OnComboBoxSelectionChanged(TSharedPtr<int32> InItem, ESelectInfo::Type SelectInfo);
	FText GetSelectedComboBoxText() const;

	TArray<TSharedPtr<VEUV::FDebugCapture>> Captures;
	FDelegateHandle CaptureAddedHandle;
	
	int32 SelectedCaptureIndex = INDEX_NONE;
	int32 LastSelectedChart = INDEX_NONE;
	EVEUVVisualizationMode CurrentMode = EVEUVVisualizationMode::UVLayout;

	/** Sample visibility filter */
	enum class ESampleFilter : uint8
	{
		None,
		All,
		Area,
		Complexity,
		Adaptive
	};
	ESampleFilter SampleFilter = ESampleFilter::All;

	bool bShowVoxels = true;
	bool bShowSettings = false;

	TSharedPtr<IStructureDetailsView> SettingsDetailsView;
	TSharedPtr<FStructOnScope> SettingsStructData;
	TSharedPtr<SBox> SettingsPanel;

	TSharedPtr<SVEUVMeshViewport> MeshViewport;
	TSharedPtr<SVEUVPlotViewport> PlotViewport;
	TSharedPtr<SCheckBox> LogScaleCheckbox;
	TSharedPtr<SBox> ViewportContainer;
	TSharedPtr<STextBlock> StatsText;
	TArray<TSharedPtr<int32>> ComboBoxSource;
	TSharedPtr<SComboBox<TSharedPtr<int32>>> BuildComboBox;
	TArray<TSharedPtr<int32>> SnapshotComboBoxSource;
	TSharedPtr<SComboBox<TSharedPtr<int32>>> SnapshotComboBox;

	/** Snapshot history selection */
	int32 SelectedSnapshotIndex = INDEX_NONE;

	/** Snapshot playback */
	float PlaybackFPS = 32.0f;
	float PlaybackAccumulator = 0.0f;
	bool bPlayingSnapshots = false;
};
