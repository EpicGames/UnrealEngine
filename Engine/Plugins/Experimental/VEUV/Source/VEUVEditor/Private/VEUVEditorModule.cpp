// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Styling/AppStyle.h"
#include "VEUVDebugPanel.h"

#define LOCTEXT_NAMESPACE "VEUVEditor"

static const FName VEUVDebugTabName("VEUVDebug");

class FVEUVEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			VEUVDebugTabName,
			FOnSpawnTab::CreateLambda([](const FSpawnTabArgs& Args) -> TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(NomadTab)
					.Label(LOCTEXT("VEUVDebugTab", "VEUV Debug"))
					[
						SNew(SVEUVDebugPanel)
					];
			}))
			.SetDisplayName(LOCTEXT("VEUVDebugTab", "VEUV Debug"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsMiscCategory()
		);
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(VEUVDebugTabName);
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FVEUVEditorModule, VEUVEditor)
