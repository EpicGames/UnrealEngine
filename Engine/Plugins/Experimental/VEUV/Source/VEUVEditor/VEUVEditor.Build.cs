// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VEUVEditor : ModuleRules
{
	public VEUVEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"VEUVCore",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Slate",
			"SlateCore",
			"UnrealEd",
			"WorkspaceMenuStructure",
			"ToolMenus",
			"InputCore",
			"PropertyEditor",
		});
	}
}
