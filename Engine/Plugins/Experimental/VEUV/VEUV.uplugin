{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "VEUV - Volume Encoded UV Maps",
	"Description": "Volume encoded UV parameterization",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VEUVCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VEUVEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}